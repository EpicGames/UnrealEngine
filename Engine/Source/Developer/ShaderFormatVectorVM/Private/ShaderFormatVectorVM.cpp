// Copyright Epic Games, Inc. All Rights Reserved.
//

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "hlslcc.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"



extern bool CompileVectorVMShader(
	const FShaderCompilerInput& Input,
	const FShaderPreprocessOutput& PreprocessOutput,
	FShaderCompilerOutput& Output);

static FName NAME_VVM_1_0(TEXT("VVM_1_0"));
static const FGuid UE_SHADER_VECTORVM_VER = FGuid("51C3EC95-421E-43B1-8AA9-61B6D4EDC1BF");

class FShaderFormatVectorVM : public UE::ShaderCompilerCommon::FBaseShaderFormat
{
	enum class VectorVMFormats : uint8
	{
		VVM_1_0,
	};

	void CheckFormat(FName Format) const
	{
		check(Format == NAME_VVM_1_0);
	}

public:
	virtual uint32 GetVersion(FName Format) const override
	{
		CheckFormat(Format);
		uint32 VVMVersion = 0;
		if (Format == NAME_VVM_1_0)
		{
			VVMVersion = (int32)VectorVMFormats::VVM_1_0;
		}
		else
		{
			check(0);
		}

		const uint16 Version = ((HLSLCC_VersionMinor & 0xff) << 8) | (VVMVersion & 0xff);

		uint32 Result = GetTypeHash(HLSLCC_VersionMinor);
		Result = HashCombine(Result, GetTypeHash(VVMVersion));
		Result = HashCombine(Result, GetTypeHash(UE_SHADER_VECTORVM_VER));

		return Result;
	}
	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_VVM_1_0);
	}

	virtual void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const override
	{
		Input.Environment.SetDefine(TEXT("COMPILER_HLSLCC"), 1);
		Input.Environment.SetDefine(TEXT("COMPILER_VECTORVM"), 1);
		Input.Environment.SetDefine(TEXT("VECTORVM_PROFILE"), 1);
		Input.Environment.SetDefine(TEXT("FORCE_FLOATS"), (uint32)1);

		// minifier has not been tested on vector VM; it's possible this could be removed to improve deduplication rate
		Input.Environment.CompilerFlags.Remove(CFLAG_RemoveDeadCode);

		// source stripping process adds comments which are not handled by this backend
		Input.Environment.CompilerFlags.Add(CFLAG_DisableSourceStripping);
	}

	virtual void CompilePreprocessedShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FShaderCompilerOutput& Output) const override
	{
		CheckFormat(Input.ShaderFormat);
		CompileVectorVMShader(Input, PreprocessOutput, Output);
	}

	virtual const TCHAR* GetPlatformIncludeDirectory() const
	{
		return TEXT("");
	}
};

/**
 * Module for VectorVM shaders
 */

static IShaderFormat* Singleton = NULL;

class FShaderFormatVectorVMModule : public IShaderFormatModule
{
public:
	virtual ~FShaderFormatVectorVMModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	virtual IShaderFormat* GetShaderFormat()
	{
		if (!Singleton)
		{
			Singleton = new FShaderFormatVectorVM();
		}
		return Singleton;
	}
};

IMPLEMENT_MODULE( FShaderFormatVectorVMModule, ShaderFormatVectorVM);
