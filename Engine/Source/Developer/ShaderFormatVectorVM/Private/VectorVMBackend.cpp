// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorVMBackend.h"
#include "ShaderFormatVectorVM.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "compiler.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

#include "VectorVM.h"

#include "HAL/IConsoleManager.h"

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("VectorVMBackend"), STATGROUP_VectorVMBackend, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Generate Main"), STAT_VVMGenerateMain, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Generate Code"), STAT_VVMGenerateCode, STATGROUP_VectorVMBackend);

DECLARE_CYCLE_STAT(TEXT("Initial/Misc"), STAT_VVMInitMisc, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Matriced To Vectors"), STAT_VVMMatToVec, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Branches To Selects"), STAT_VVMBranchesToSelects, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("To Single Op"), STAT_VVMToSingleOp, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Scalarize"), STAT_VVMScalarize, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Merge Ops"), STAT_VVMMergeOps, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Propagate non-expressions"), STAT_VVMPropagateNonExprs, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Cleanup"), STAT_VVMCleanup, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Validate"), STAT_VVMValidate, STATGROUP_VectorVMBackend);
DECLARE_CYCLE_STAT(TEXT("Gen Bytecode"), STAT_VVMGenByteCode, STATGROUP_VectorVMBackend);


#if !PLATFORM_WINDOWS
#define _strdup strdup
#endif

DEFINE_LOG_CATEGORY(LogVVMBackend);

bool FVectorVMCodeBackend::GenerateMain(EHlslShaderFrequency Frequency, const char* EntryPoint, exec_list* Instructions, _mesa_glsl_parse_state* ParseState)
{
	SCOPE_CYCLE_COUNTER(STAT_VVMGenerateMain);
	//vm_debug_dump(Instructions, ParseState);

	ir_function_signature* MainSig = nullptr;
	int NumFunctions = 0;

	foreach_iter(exec_list_iterator, iter, *Instructions)
	{
		ir_instruction *ir = (ir_instruction *)iter.get();
		ir_function* Function = ir->as_function();
		if (Function)
		{
			if (strcmp(Function->name, "SimulateMain") == 0)
			{
				foreach_iter(exec_list_iterator, sigiter, *Function)
				{
					ir_function_signature* Sig = (ir_function_signature *)sigiter.get();
					Sig->is_main = true;
				}
			}
		}
	}

	//vm_debug_dump(Instructions, ParseState);
	return true;
}

/** Finds all external function calls (those with no body) and make them built in. This is to bypass issues with constant propagation visitor. */
class ir_make_external_funcs_builtin : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state *parse_state;
	ir_make_external_funcs_builtin(_mesa_glsl_parse_state *in_state) : parse_state(in_state) {	}
	virtual ~ir_make_external_funcs_builtin() {	}

	virtual ir_visitor_status visit_enter(ir_function_signature *sig)
	{
		if (sig->body.get_head() == nullptr)
		{
			sig->is_builtin = true;
		}
		return visit_continue;
	}

public:
	static void run(exec_list *ir, _mesa_glsl_parse_state *state)
	{
		ir_make_external_funcs_builtin visitor(state);
		visit_list_elements(&visitor, ir);
	}
};

/** Removes any stat scopes that are now empty due to other optimizations. */
class ir_remove_empty_stat_scopes : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state *parse_state;
	ir_remove_empty_stat_scopes(_mesa_glsl_parse_state *in_state)
		: parse_state(in_state)
		, progress(false)
	{
	}
	virtual ~ir_remove_empty_stat_scopes() {	}

	TArray<ir_call*> EnterStatScopeCalls;
	TArray<int32> StatScopeAssingmentCounts;

	bool progress;

	virtual ir_visitor_status visit_enter(ir_call* call)
	{
		if (strcmp(call->callee_name(), "EnterStatScope") == 0)
		{
			EnterStatScopeCalls.Push(call);
			StatScopeAssingmentCounts.Push(0);			
		}
		else if (strcmp(call->callee_name(), "ExitStatScope") == 0)
		{
			check(EnterStatScopeCalls.Num() == StatScopeAssingmentCounts.Num());
			if (EnterStatScopeCalls.Num() == 0)
			{
				_mesa_glsl_error(parse_state, "Mismatched EnterStatScope/ExitStatScope calls.");
				return visit_stop;
			}

			ir_call* EnterCall = EnterStatScopeCalls.Pop();
			int32 NumAssignments = StatScopeAssingmentCounts.Pop();
			if (NumAssignments == 0)
			{
				EnterCall->remove();
				call->remove();
			}
			else if (StatScopeAssingmentCounts.Num() > 0)
			{
				// On leaving a non-empty scope, we add their assignments to the outer scope.
				// Without this, nested scopes can be removed although they are not empty.
				StatScopeAssingmentCounts.Last() += NumAssignments;
			}
		}
		else if (StatScopeAssingmentCounts.Num() > 0)
		{
			++StatScopeAssingmentCounts.Last();
		}
		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign)
	{
		if (StatScopeAssingmentCounts.Num() > 0)
		{
			++StatScopeAssingmentCounts.Last();
		}
		return visit_continue_with_parent;
	}

public:

	static void run(exec_list *ir, _mesa_glsl_parse_state *state)
	{
		ir_remove_empty_stat_scopes visitor(state);
		do
		{
			visitor.progress = false;
			visit_list_elements(&visitor, ir);
		} while (visitor.progress);
	}
};

char* FVectorVMCodeBackend::GenerateCode(exec_list* ir, _mesa_glsl_parse_state* state, EHlslShaderFrequency Frequency)
{
	SCOPE_CYCLE_COUNTER(STAT_VVMGenerateCode); 
	vm_debug_print("========VECTOR VM BACKEND: Generate Code==============\n");
	vm_debug_dump(ir, state);

	if (state->error) return nullptr;

	bool progress = false;

	ir_make_external_funcs_builtin::run(ir, state);

	FlattenUniformBufferStructures(ir, state);

	// The vector VM doesn't support dynamic loops, and the codegen panics if it sees loops. We'll just set the unroll threshold
	// to a very large value, to make sure loops are always unrolled.
	state->maxunrollcount = 16384;

	state->conservative_propagation = false;
	{
		SCOPE_CYCLE_COUNTER(STAT_VVMInitMisc);
		vm_debug_print("== Initial misc ==\n");
		do
		{
// 			progress = do_function_inlining(ir);//Full optimization pass earlier will have already done this.
			progress = do_mat_op_to_vec(ir);
			progress = do_vec_op_to_scalar(ir, state) || progress;
			progress = do_vec_index_to_swizzle(ir) || progress;

			// slip in this call to extract scalar constants out of swizzled constant vectors
			progress = vm_extract_scalar_ops(ir, state) || progress;

			progress = do_optimization_pass(ir, state, true) || progress;
			vm_debug_print("======== Optimization Pass ==============\n");
			vm_debug_dump(ir, state);
		} while (progress);
		//validate_ir_tree(ir, state);
		if (state->error) return nullptr;
	}
	
	{
		SCOPE_CYCLE_COUNTER(STAT_VVMMatToVec);
		//99% complete code to remove all matrices from the code and replace them with just swizzled vectors. 
		//For now visitors below here can handle matrices ok but we may hit some edge cases in future requiring their removal.
		vm_debug_print("== matrices to vectors ==\n");
		vm_matrices_to_vectors(ir, state);
		vm_debug_dump(ir, state);
		if (state->error) return nullptr;
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMBranchesToSelects);
		vm_debug_print("== Branches to selects ==\n");
		vm_flatten_branches_to_selects(ir, state);
		//validate_ir_tree(ir, state);
		vm_debug_dump(ir, state);
	}

	{

		SCOPE_CYCLE_COUNTER(STAT_VVMToSingleOp);
		vm_debug_print("== To Single Op ==\n");
		vm_to_single_op(ir, state);
		//validate_ir_tree(ir, state);
		vm_debug_dump(ir, state);
		if (state->error) return nullptr;
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMScalarize);
		vm_debug_print("== Scalarize ==\n");
		vm_scalarize_ops(ir, state);
		//validate_ir_tree(ir, state);
		vm_debug_dump(ir, state);
		//validate_ir_tree(ir, state);
		if (state->error) return nullptr;
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMScalarize);
		vm_debug_print("== Scalar Optimization ==\n");
		progress = false;
		do
		{
			progress = do_algebraic(state, ir);
		} while (progress);
		vm_debug_dump(ir, state);
		if (state->error) return nullptr;
	}
	{
		SCOPE_CYCLE_COUNTER(STAT_VVMMergeOps);
		vm_debug_print("== Merge Ops ==\n");
		vm_merge_ops(ir, state);
	//	validate_ir_tree(ir, state);
		vm_debug_dump(ir, state);
		//validate_ir_tree(ir, state);
		if (state->error) return nullptr;
	}

	{

		SCOPE_CYCLE_COUNTER(STAT_VVMPropagateNonExprs);
		vm_debug_print("== Propagate non-expressions ==\n");
		vm_propagate_non_expressions_visitor(ir, state);
		//validate_ir_tree(ir, state);
		vm_debug_dump(ir, state);
		if (state->error) return nullptr;
	}

	{
		//SCOPE_CYCLE_COUNTER(STAT_VVMCleanup);
		vm_debug_print("== Remove self ref assignments ==\n");
		vm_remove_self_ref_assignments(ir, state);
		vm_debug_dump(ir, state);
		if (state->error) return nullptr;
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMCleanup);
		vm_debug_print("== Cleanup ==\n");
		// Final cleanup
		progress = false;
		do
		{
			progress = do_dead_code(ir, false);
			progress = do_dead_code_local(ir) || progress;
			progress = do_swizzle_swizzle(ir) || progress;
			progress = do_noop_swizzle(ir) || progress;
			progress = do_copy_propagation(ir) || progress;
			progress = do_copy_propagation_elements(ir) || progress;
			progress = do_constant_propagation(ir) || progress;
		} while (progress);

		ir_remove_empty_stat_scopes::run(ir, state);
		vm_debug_dump(ir, state);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMValidate);
		validate_ir_tree(ir, state);
	}

	if (state->error) return nullptr;

	{
		SCOPE_CYCLE_COUNTER(STAT_VVMGenByteCode);
		vm_gen_bytecode(ir, state, CompilationOutput);
	}

	return  nullptr;// Cheat and emit the bytecode into the CompilationOutput. The return here is treated as a string so the 0's it contains prove problematic.
}

void FVectorVMLanguageSpec::SetupLanguageIntrinsics(_mesa_glsl_parse_state* State, exec_list* ir)
{
// 	//TODO: Need to add a way of stopping these being stripped if they're not used in the code.
// 	//We fine if the func is unused entirely but we need to keep the scalar signatures for when we scalarize the call.
// 	//Maybe we can just keep the wrong sig but still replace the ret val and params?
// 	make_intrinsic_genType(ir, State, "mad", ir_invalid_opcode, IR_INTRINSIC_FLOAT, 3, 1, 4);
	make_intrinsic_genType(ir, State, "rand", ir_invalid_opcode, IR_INTRINSIC_FLOAT, 1, 1, 4);
	make_intrinsic_genType(ir, State, "rand", ir_invalid_opcode, IR_INTRINSIC_INT, 1, 1, 4);
	make_intrinsic_genType(ir, State, "Modulo", ir_invalid_opcode, IR_INTRINSIC_FLOAT, 1, 1, 4);

	make_intrinsic_genType(ir, State, "EnterStatScope", ir_invalid_opcode, IR_INTRINSIC_RETURNS_VOID | IR_INTRINSIC_INT, 1, 1, 1);
	make_intrinsic_genType(ir, State, "ExitStatScope", ir_invalid_opcode, IR_INTRINSIC_RETURNS_VOID, 0, 0, 0);

// Dont need all these as we're only using the basic scalar function which we provide the signature for in the usf.
// 	make_intrinsic_genType(ir, State, "InputDataFloat", ir_invalid_opcode, IR_INTRINSIC_FLOAT, 2, 1, 1);
// 	make_intrinsic_genType(ir, State, "InputDataInt", ir_invalid_opcode, IR_INTRINSIC_INT, 2, 1, 1);
// 	make_intrinsic_genType(ir, State, "OutputDataFloat", ir_invalid_opcode, 0, 3, 1, 1);
// 	make_intrinsic_genType(ir, State, "OutputDataInt", ir_invalid_opcode, 0, 3, 1, 1);
//	make_intrinsic_genType(ir, State, "AcquireIndex", ir_invalid_opcode, IR_INTRINSIC_INT, 2, 1, 1);
}

static int32 GbDumpVMIR = 0;
static FAutoConsoleVariableRef CVarDumpVMIR(
	TEXT("fx.DumpVMIR"),
	GbDumpVMIR,
	TEXT("If > 0 verbose logging is enabled for the vm compiler backend. \n"),
	ECVF_Default
);

void DebugDumpIR(struct exec_list* ir, struct _mesa_glsl_parse_state* State)
{
	if (GbDumpVMIR)
	{
		IRDump(ir, State);
	}
}
