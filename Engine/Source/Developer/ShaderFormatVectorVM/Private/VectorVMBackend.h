// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "hlslcc.h"
#include "LanguageSpec.h"
#include "VectorVM.h"

struct FVectorVMCompilationOutput;

class FVectorVMLanguageSpec : public ILanguageSpec
{
protected:

public:
	virtual bool LowerSubToAddNeg() const override { return false; }
	virtual bool SupportsFusedMultiplyAdd() const override { return true; }
	virtual bool SupportsDeterminantIntrinsic() const override { return false; }
	virtual bool SupportsTransposeIntrinsic() const override{ return false;	}
	virtual bool SupportsIntegerModulo() const override { return false;	}
	virtual bool SupportsMatrixConversions() const override { return false; }

	//#todo-rco: Enable
	virtual bool AllowsSharingSamplers() const override { return false; }

	FVectorVMLanguageSpec() {}

	//TODO

	virtual void SetupLanguageIntrinsics(_mesa_glsl_parse_state* State, exec_list* ir) override;
};

class ir_variable;

// Generates VectorVM compliant code from IR tokens
struct FVectorVMCodeBackend : public FCodeBackend
{
	FVectorVMCodeBackend(unsigned int InHlslCompileFlags, EHlslCompileTarget InTarget, FVectorVMCompilationOutput& InCompilerOutput) :
		FCodeBackend(InHlslCompileFlags, InTarget), CompilationOutput(InCompilerOutput)
	{
	}


	virtual bool GenerateMain(EHlslShaderFrequency Frequency, const char* EntryPoint, exec_list* Instructions, _mesa_glsl_parse_state* ParseState);

	virtual char* GenerateCode(struct exec_list* ir, struct _mesa_glsl_parse_state* ParseState, EHlslShaderFrequency Frequency) override;

	//TODO: We'll likely need to do this 
	// Return false if there were restrictions that made compilation fail
	//virtual bool ApplyAndVerifyPlatformRestrictions(exec_list* Instructions, _mesa_glsl_parse_state* ParseState, EHlslShaderFrequency Frequency) override;

	//TODO: Do we need to generate a main()?
	//virtual bool GenerateMain(EHlslShaderFrequency Frequency, const char* EntryPoint, exec_list* Instructions, _mesa_glsl_parse_state* ParseState) override;

	FVectorVMCompilationOutput& CompilationOutput;
};

class ir_call;
struct exec_list;
struct _mesa_glsl_parse_state;

enum class ECallScalarizeMode : uint8
{
	SplitCalls,//simple calls for things like Random(), split the call into separate scalar calls
	SplitParams,//External function calls, split all the params up into a new function
	None,
	Error,
};

ECallScalarizeMode get_scalarize_mode(ir_function_signature* in_sig);
EVectorVMOp get_special_vm_opcode(ir_function_signature* signature);

void vm_remove_self_ref_assignments(exec_list* instructions, _mesa_glsl_parse_state *state);
void vm_matrices_to_vectors(exec_list* instructions, _mesa_glsl_parse_state *state);
bool do_vec_op_to_scalar(exec_list *instructions, _mesa_glsl_parse_state* parse_state);
bool vm_flatten_branches_to_selects(exec_list *instructions, _mesa_glsl_parse_state *state);
void vm_to_single_op(exec_list *ir, _mesa_glsl_parse_state *state);
void vm_merge_ops(exec_list *ir, _mesa_glsl_parse_state *state);
void vm_scalarize_ops(exec_list* ir, _mesa_glsl_parse_state* state);
bool vm_extract_scalar_ops(exec_list* ir, _mesa_glsl_parse_state* state);
void vm_propagate_non_expressions_visitor(exec_list* ir, _mesa_glsl_parse_state* state);
void vm_gen_bytecode(exec_list *ir, _mesa_glsl_parse_state *state, FVectorVMCompilationOutput& InCompOutput);


void DebugDumpIR(struct exec_list* ir, struct _mesa_glsl_parse_state* State);

//////////////////////////////////////////////////////////////////////////
//Enable verbose debug dumps.
#define VM_VERBOSE_LOGGING 0
#if VM_VERBOSE_LOGGING == 2
#define vm_debug_dump(ir, state) DebugDumpIR(ir, state)
#define vm_debug_print dprintf
#elif VM_VERBOSE_LOGGING == 1
#define vm_debug_dump(ir, state) 
#define vm_debug_print dprintf
#else
#define vm_debug_dump(ir, state)
#define vm_debug_print(...)
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogVVMBackend, All, All);