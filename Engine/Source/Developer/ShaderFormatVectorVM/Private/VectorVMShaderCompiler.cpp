// Copyright Epic Games, Inc. All Rights Reserved.
// .

#include "CoreMinimal.h"
#include "CrossCompiler.h"
#include "CrossCompilerCommon.h"
#include "HAL/CriticalSection.h"
#include "Misc/FileHelper.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCore.h"
#include "ShaderFormatVectorVM.h"
#include "ShaderPreprocessor.h"
#include "ShaderPreprocessTypes.h"
#include "ShaderCompilerDefinitions.h"

#include "VectorVM.h"
#include "VectorVMBackend.h"
#include "VectorVMTestCompile.h"

DEFINE_LOG_CATEGORY_STATIC(LogVectorVMShaderCompiler, Log, All); 

DECLARE_STATS_GROUP(TEXT("VectorVM"), STATGROUP_VectorVM, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("VectorVM - Compiler - CompileShader_VectorVM"), STAT_VectorVM_Compiler_CompileShader_VectorVM, STATGROUP_VectorVM);
DECLARE_CYCLE_STAT(TEXT("VectorVM - Compiler - PreprocessShader"), STAT_VectorVM_Compiler_CompileShader_VectorVMPreprocessShader, STATGROUP_VectorVM);
DECLARE_CYCLE_STAT(TEXT("VectorVM - Compiler - CrossCompilerContextRun"), STAT_VectorVM_Compiler_CompileShader_CrossCompilerContextRun, STATGROUP_VectorVM);

bool CompileVectorVMShader(
	const FShaderCompilerInput& Input,
	const FShaderPreprocessOutput& PreprocessOutput,
	FVectorVMCompilationOutput& VMCompilationOutput,
	bool bSkipBackendOptimizations)
{
	SCOPE_CYCLE_COUNTER(STAT_VectorVM_Compiler_CompileShader_VectorVM);

	EHlslCompileTarget HlslCompilerTarget = HCT_FeatureLevelSM5;
	ECompilerFlags PlatformFlowControl = CFLAG_AvoidFlowControl;

	char* ShaderSource = NULL;
	char* ErrorLog = NULL;

	const EHlslShaderFrequency Frequency = HSF_VertexShader;

	uint32 CCFlags = HLSLCC_NoPreprocess;

	if (bSkipBackendOptimizations)
	{
		CCFlags |= HLSLCC_DisableBackendOptimizations;
	}

	FVectorVMCodeBackend VVMBackEnd(CCFlags, HlslCompilerTarget, VMCompilationOutput);
	FVectorVMLanguageSpec VVMLanguageSpec;
	FAnsiStringView Source = PreprocessOutput.GetSourceViewAnsi();

	bool bResult = false;
	{
		FScopeLock HlslCcLock(CrossCompiler::GetCrossCompilerLock());
		FHlslCrossCompilerContext CrossCompilerContext(CCFlags, Frequency, HlslCompilerTarget);
		if (CrossCompilerContext.Init(TCHAR_TO_ANSI(*Input.VirtualSourceFilePath), &VVMLanguageSpec))
		{
			SCOPE_CYCLE_COUNTER(STAT_VectorVM_Compiler_CompileShader_CrossCompilerContextRun);

			bResult = CrossCompilerContext.Run(
				Source.GetData(),
				TCHAR_TO_ANSI(*Input.EntryPointName),
				&VVMBackEnd,
				&ShaderSource,
				&ErrorLog);
		}
	}

	if (ErrorLog)
	{
		int32 SrcLen = FPlatformString::Strlen(ErrorLog);
		int32 DestLen = FPlatformString::ConvertedLength<TCHAR, ANSICHAR>(ErrorLog, SrcLen);
		TArray<TCHAR> Converted;
		Converted.AddUninitialized(DestLen);

		FPlatformString::Convert<ANSICHAR, TCHAR>(Converted.GetData(), DestLen, ErrorLog, SrcLen);
		Converted.Add(0);

		VMCompilationOutput.Errors = Converted.GetData();

		// Dump preprocessed and generated source to debug info folder (or artifacts)
		FString VectorVMPreprocessedLines;
		PreprocessOutput.ForEachLine(
			[&VectorVMPreprocessedLines](FAnsiStringView Line, int32 LineIndex)
			{
				VectorVMPreprocessedLines.Appendf(TEXT("/*%d*/%.*hs"), LineIndex, Line.Len(), Line.GetData());
			}
		);
		const FString DebugInfoPath = Input.GetOrCreateShaderDebugInfoPath();
		FFileHelper::SaveStringToFile(VectorVMPreprocessedLines, *FPaths::Combine(DebugInfoPath, TEXT("VectorVMPreprocessOutput.txt")));

		if (ShaderSource)
		{
			FFileHelper::SaveStringToFile(ANSI_TO_TCHAR(ShaderSource), *FPaths::Combine(DebugInfoPath, TEXT("VectorVMShaderOutput.usf")));
		}

		UE_LOGF(LogVectorVMShaderCompiler, Warning, "Warnings while processing %ls", *Input.DebugGroupName);
		UE_LOGF(LogVectorVMShaderCompiler, Warning, "%ls", *VMCompilationOutput.Errors);

		free(ErrorLog);
	}
	else
	{
		VMCompilationOutput.Errors.Empty();
	}

	if (ShaderSource)
	{
		free(ShaderSource);
	}

	return bResult;
}

bool CompileVectorVMShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FShaderCompilerOutput& Output)
{
	FVectorVMCompilationOutput CompilationOutput;
	bool bResult = CompileVectorVMShader(Input, PreprocessOutput, CompilationOutput, Input.Environment.CompilerFlags.Contains(CFLAG_SkipOptimizations));
	
	if (bResult)
	{
		FMemoryWriter Ar(Output.ShaderCode.GetWriteAccess(), true);
		Ar << CompilationOutput;
		Output.bSucceeded = true;
	}
	else if (CompilationOutput.Errors.Len() > 0)
	{
		Output.Errors.Add(FShaderCompilerError(*CompilationOutput.Errors));
	}
	return bResult;
}

bool TestCompileVectorVMShader(
	const FShaderCompilerInput& Input,
	FVectorVMCompilationOutput& VMCompilationOutput,
	bool bSkipBackendOptimizations)
{
	FShaderPreprocessOutput PreprocessOutput;
	if (!UE::ShaderCompilerCommon::ExecuteShaderPreprocessingSteps(PreprocessOutput, Input, Input.Environment))
	{
		if (PreprocessOutput.GetErrors().Num() != 0)
		{
			FString Errors;
			for (const FShaderCompilerError& Error : PreprocessOutput.GetErrors())
			{
				Errors += Error.GetErrorString() + TEXT("\r\n");
			}
			VMCompilationOutput.Errors = Errors;
		}
		return false;
	}

	return CompileVectorVMShader(Input, PreprocessOutput, VMCompilationOutput, bSkipBackendOptimizations);
}
