// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

// evaluate expressions and see if they are operating a swizzle of a constant; if they are just
// replace it with the scalar constant
class ir_extract_scalar_visitor : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state* parse_state;
	bool has_changed;

	ir_extract_scalar_visitor(_mesa_glsl_parse_state* in_state)
		: parse_state(in_state)
		, has_changed(false)
	{
	}

	virtual ir_visitor_status visit_enter(ir_expression* expression)
	{
		for (unsigned int op_it = 0; op_it < expression->get_num_operands(); ++op_it)
		{
			if (ir_swizzle* swizzle = expression->operands[op_it]->as_swizzle())
			{
				if (swizzle->mask.num_components == 1)
				{
					if (ir_constant* child_constant = swizzle->val->as_constant())
					{
						ir_constant* scalar_constant = new(parse_state) ir_constant(child_constant, swizzle->mask.x);
						expression->operands[op_it] = scalar_constant;
					}
				}
			}
		}

		return visit_continue;
	}

public:
	static bool run(exec_list* ir, _mesa_glsl_parse_state* state)
	{
		bool has_changed = false;
		bool progress = false;

		do 
		{
			ir_extract_scalar_visitor extract_scalar(state);
			visit_list_elements(&extract_scalar, ir);
			progress = extract_scalar.has_changed;
			has_changed = has_changed || progress;
		} while (progress);

		return has_changed;
	}
};

bool vm_extract_scalar_ops(exec_list* ir, _mesa_glsl_parse_state* state)
{
	return ir_extract_scalar_visitor::run(ir, state);
}