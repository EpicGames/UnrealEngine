// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"
#include "ir_vm_helper.h"

//Helper visitor to replace any flatten branches (currently all) with selection statements the VM can deal with.
class ir_flatten_branch_to_select_visitor final : public ir_rvalue_visitor
{
public:

	_mesa_glsl_parse_state* parse_state;
	ir_if* curr_if;
	TArray<ir_assignment*>* curr_assignments;
	TArray<ir_assignment*> a_assignments;
	TArray<ir_assignment*> b_assignments;

	// keeps track of the rvalues created (branch_flattened) to hold the temporary value calculated within a then/else block
	TArray<ir_rvalue*>* curr_branch_flattened_rvalues;
	TArray<ir_rvalue*> if_flattened_rvalues;
	TArray<ir_rvalue*> then_flattened_rvalues;

	TMap<const glsl_type*, ir_function_signature*> select_functions;

	void generate_select_signature(ir_function* func, unsigned base_type, unsigned num_components)
	{
		const glsl_type* select_type = glsl_type::get_instance(base_type, num_components, 1);
		ir_function_signature* new_sig = new(parse_state)ir_function_signature(select_type);
		new_sig->is_builtin = true;
		new_sig->has_output_parameters = false;
		new_sig->parameters.push_tail(new(parse_state)ir_variable(glsl_type::bool_type, "condition", ir_var_in));
		new_sig->parameters.push_tail(new(parse_state)ir_variable(select_type, "a", ir_var_in));
		new_sig->parameters.push_tail(new(parse_state)ir_variable(select_type, "b", ir_var_in));
		func->add_signature(new_sig);
	}

	ir_flatten_branch_to_select_visitor(_mesa_glsl_parse_state* in_parse_state, exec_list* ir)
		: parse_state(in_parse_state)
	{
		curr_if = nullptr;
		curr_assignments = nullptr;
		curr_branch_flattened_rvalues = nullptr;

		ir_function* select_func = new(parse_state)ir_function("select");
		generate_select_signature(select_func, GLSL_TYPE_BOOL, 1);
		for (unsigned i = 1; i <= 4; ++i)
		{
			generate_select_signature(select_func, GLSL_TYPE_FLOAT, i);
			generate_select_signature(select_func, GLSL_TYPE_INT, i);
		}
		ir->push_tail(select_func);

		foreach_iter(exec_list_iterator, Iter, *ir)
		{
			ir_instruction *inst = (ir_instruction *)Iter.get();
			ir_function *Function = inst->as_function();
			if (Function && strcmp(Function->name, "select") == 0)
			{
				foreach_iter(exec_list_iterator, SigIter, *Function)
				{
					ir_function_signature* sig = (ir_function_signature *)SigIter.get();
					select_functions.Add(sig->return_type, sig);
				}
			}
		}
	}

	virtual ~ir_flatten_branch_to_select_visitor()
	{
	}

	ir_function_signature* get_select_signature(const glsl_type* type)
	{
		if (ir_function_signature** found = select_functions.Find(type))
		{
			return *found;
		}

		_mesa_glsl_error(parse_state, "Invalid select() signature requested! %s.", type->name);
		return nullptr;
	}

	ir_visitor_status visit_leave(ir_if * ir)
	{
		//Eventually leave real branches intact but for now we flatten all branches to selects.
		// 		if (ir->mode == if_branch)
		// 			return visit_continue;

		check(base_ir->next && base_ir->prev);

		if (parse_state->error)
			return visit_stop;

		//Here we do the following to flatten the branch to selects.
		//1. Replace all variables in the block with new temporaries.
		//2. Combine both blocks into the outer ir
		//3. Add assignments for each value from it's new temporary to the end of the ir
		//4. Search for matching assignment LHSs and replace them with a selection using the if condition. Any non-matching assignments are replaced with selections between the new temp and the original.

		ir_if* old_if = curr_if;
		curr_if = ir;
		//void* parent = ralloc_parent(curr_if);

		//Pull out our condition to a variable.
		ir_variable *condition_var = new(parse_state)ir_variable(glsl_type::bool_type,
			"branch_flatten_condition",
			ir_var_temporary);
		base_ir->insert_before(condition_var);
		base_ir->insert_before(new(parse_state)ir_assignment(new(parse_state)ir_dereference_variable(condition_var), curr_if->condition));
		curr_if->condition = new(parse_state)ir_dereference_variable(condition_var);

		check(a_assignments.Num() == 0);
		check(curr_assignments == nullptr);
		check(curr_branch_flattened_rvalues == nullptr);
		check(if_flattened_rvalues.Num() == 0);
		curr_assignments = &a_assignments;
		curr_branch_flattened_rvalues = &if_flattened_rvalues;
		visit_list_elements(this, &curr_if->then_instructions, true);
		base_ir->insert_before(&curr_if->then_instructions);
		if_flattened_rvalues.Reset();

		check(b_assignments.Num() == 0);
		check(then_flattened_rvalues.Num() == 0);
		curr_assignments = &b_assignments;
		curr_branch_flattened_rvalues = &then_flattened_rvalues;
		visit_list_elements(this, &curr_if->else_instructions, true);
		base_ir->insert_before(&curr_if->else_instructions);
		then_flattened_rvalues.Reset();

		curr_branch_flattened_rvalues = nullptr;

		TArray<bool> HandledBAssignments;
		HandledBAssignments.SetNumZeroed(b_assignments.Num());

		for(ir_assignment* assign : a_assignments)
		{
			ir_rvalue* selection_other = nullptr;
			int32 selection_other_idx = b_assignments.IndexOfByPredicate([&](ir_assignment* other_assign) { return AreEquivalent(assign->lhs, other_assign->lhs); });
			if (selection_other_idx == INDEX_NONE)
			{
				//Couldn't find a matching assignment in the other path to use as a selection B param.
				//Use the original value instead.
				selection_other = assign->lhs;
			}
			else
			{
				//We did find a matching statement, select using that and remove it from the b_assignments.
				selection_other = b_assignments[selection_other_idx]->rhs;
				HandledBAssignments[selection_other_idx] = true;
			}

			//Selection result
			ir_variable *result = new(parse_state)ir_variable(assign->lhs->type, "selction_result", ir_var_temporary);
			base_ir->insert_before(result);

			exec_list select_params;
			select_params.push_tail(ir->condition->clone(parse_state, nullptr));
			select_params.push_tail(assign->rhs->clone(parse_state, nullptr));
			select_params.push_tail(selection_other->clone(parse_state, nullptr));

			ir_function_signature* selection_sig = get_select_signature(assign->lhs->type);
			if (selection_sig)
			{
				ir_call* select_call = new(parse_state)ir_call(selection_sig, new(parse_state)ir_dereference_variable(result), &select_params);
				base_ir->insert_before(select_call);
			}
			assign->rhs = new(parse_state)ir_dereference_variable(result);
			base_ir->insert_before(assign);
		}

		//Deal with any remaining work done on the b path.
		for(int32 i=0; i < b_assignments.Num(); ++i)
		{
			//bail if this assignment was already handled.
			if(HandledBAssignments[i])
				continue;

			ir_assignment* assign = b_assignments[i];

			//Selection result
			ir_variable *result = new(parse_state)ir_variable(assign->lhs->type, "selction_result", ir_var_temporary);
			base_ir->insert_before(result);

			exec_list select_params;
			select_params.push_tail(ir->condition->clone(parse_state, nullptr));
			select_params.push_tail(assign->lhs->clone(parse_state, nullptr));
			select_params.push_tail(assign->rhs->clone(parse_state, nullptr));

			ir_function_signature* selection_sig = get_select_signature(assign->lhs->type);
			if (selection_sig)
			{
				ir_call* select_call = new(parse_state)ir_call(selection_sig, new(parse_state)ir_dereference_variable(result), &select_params);
				base_ir->insert_before(select_call);
			}
			assign->rhs = new(parse_state)ir_dereference_variable(result);
			base_ir->insert_before(assign);
		}

		a_assignments.Reset();
		b_assignments.Reset();

		base_ir->remove();
		curr_assignments = nullptr;
		curr_if = old_if;

		return visit_continue;
	}

	ir_dereference_variable* replace_assigned_val_with_temp(ir_dereference* val)
	{
		check(val);
		//void* parent = ralloc_parent(val);
		ir_variable *var = new(parse_state)ir_variable(val->type,
			"branch_flatten_temp",
			ir_var_temporary);
		base_ir->insert_before(var);

		ir_dereference_variable* new_deref_var = new(parse_state)ir_dereference_variable(var);

		curr_branch_flattened_rvalues->Add(new_deref_var);

		return new_deref_var;
	}

	// check if the LHS of any of our current assignments are using the current rvalue, if so we'll 
	// need to replace it with the temporary that we've stashed in curr_branch_flattened_rvalues
	// Example code that this is trying to resolve:
	// if (Condition)
	// {
	//   Foo = Bar;					// this would have been replaced with:	branch_flattened_temp0 = Bar;
	//   Foo2 = Foo;				// this would have been replaced with:	branch_flattened_temp1 = Foo; this should have been replaced by:
	//																		branch_flattened_temp1 = branch_flattened_temp0;
	// }
	virtual void handle_rvalue(ir_rvalue** rvalue) override
	{
		if (rvalue && *rvalue && curr_if && curr_assignments && !curr_assignments->IsEmpty())
		{
			check(curr_branch_flattened_rvalues->Num() == curr_assignments->Num());

			uint32 ComponentIndex = 0;
			if (ir_variable* Variable = ir_vm_helper::get_rvalue_variable(*rvalue, ComponentIndex))
			{
				int32 AssignmentCount = curr_assignments->Num();

				for (int32 AssignIt = 0; AssignIt < AssignmentCount; ++AssignIt)
				{
					ir_assignment* prev_assign = (*curr_assignments)[AssignIt];

					uint32 AssignComponentIndex = 0;
					if (ir_variable* prev_assign_variable = ir_vm_helper::get_rvalue_variable(prev_assign->lhs, AssignComponentIndex))
					{
						if (prev_assign_variable == Variable && AssignComponentIndex == ComponentIndex)
						{
							ir_rvalue* new_rval = (*curr_branch_flattened_rvalues)[AssignIt]->clone(parse_state, nullptr);
							(*rvalue) = new_rval;
						}
					}
				}
			}
		}
	}

	ir_visitor_status visit_leave(ir_assignment * assign)
	{
		if (curr_if)
		{
			check(curr_assignments);

			//void* parent = ralloc_parent(assign);
			ir_dereference_variable* new_deref = replace_assigned_val_with_temp(assign->lhs);

			ir_assignment* new_assign = new(parse_state)ir_assignment(assign->lhs, new_deref);
			assign->set_lhs(new_deref);
			curr_assignments->Add(new_assign);
		}

		return visit_continue;
	}

	ir_visitor_status visit_leave(ir_call* call)
	{
		if (curr_if)
		{
			check(curr_assignments);

			//void* parent = ralloc_parent(call);
			if (call->return_deref)
			{
				ir_dereference_variable* new_deref = replace_assigned_val_with_temp(call->return_deref);
				ir_assignment* new_assign = new(parse_state)ir_assignment(call->return_deref, new_deref);
				call->return_deref = new_deref;
				curr_assignments->Add(new_assign);
			}

			ir_rvalue* actual_param = (ir_rvalue*)call->actual_parameters.get_head();
			foreach_iter(exec_list_iterator, iter, call->callee->parameters)
			{
				ir_variable *var = (ir_variable *)iter.get();
				if (var->mode == ir_var_out || var->mode == ir_var_inout)
				{
					ir_dereference_variable* new_deref = replace_assigned_val_with_temp(actual_param->as_dereference());
					ir_assignment* new_assign = new(parse_state)ir_assignment(actual_param, new_deref);
					check(actual_param->next && actual_param->prev);
					actual_param->replace_with(new_deref);
					curr_assignments->Add(new_assign);
				}
				actual_param = (ir_rvalue*)actual_param->next;
			}
		}

		return visit_continue;
	}
};

bool vm_flatten_branches_to_selects(exec_list *ir, _mesa_glsl_parse_state *state)
{
	ir_flatten_branch_to_select_visitor visitor(state, ir);
	visit_list_elements(&visitor, ir, true);

	bool progress;

	do
	{
		progress = do_dead_code(ir, false);
		progress = do_dead_code_local(ir) || progress;
	} while (progress);

	return true;
}