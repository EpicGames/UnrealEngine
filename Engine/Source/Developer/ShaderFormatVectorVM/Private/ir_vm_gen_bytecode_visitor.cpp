// Copyright Epic Games, Inc. All Rights Reserved.


#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

#include "VectorVM.h"

//////////////////////////////////////////////////////////////////////////

unsigned get_dest_comp(unsigned write_mask)
{
	if (write_mask == (1 << 0)) return 0;
	if (write_mask == (1 << 1)) return 1;
	if (write_mask == (1 << 2)) return 2;
	if (write_mask == (1 << 3)) return 3;
	check(0);
	return 0;
}

bool is_supported_base_type(const glsl_type* in_type)
{
	check(in_type->is_scalar());
	return in_type->is_float() || in_type->is_integer() || in_type->is_boolean();
}

struct FVMExpresssionInfo
{
	FVMExpresssionInfo(EVectorVMOp InOp, const glsl_type* InRet, const glsl_type* InArg0, const glsl_type* InArg1 = nullptr, const glsl_type* InArg2 = nullptr, const glsl_type* InArg3 = nullptr) : Op(InOp), Ret(InRet)
	{
		Operands[0] = InArg0;
		Operands[1] = InArg1;
		Operands[2] = InArg2;
		Operands[3] = InArg3;
	}

	bool operator==(const ir_expression* expr)const
	{
		if (expr->type == Ret)
		{
			for (unsigned i = 0; i < expr->get_num_operands(); ++i)
			{
				if (expr->operands[i]->type != Operands[i])
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

	EVectorVMOp Op;
	const glsl_type* Ret;
	const glsl_type* Operands[4];
};

static TMap<ir_expression_operation, TArray<FVMExpresssionInfo>> VMExpressionMap;
void BuildExpressionMap()
{
	if (VMExpressionMap.Num() == 0)
	{
		TArray<FVMExpresssionInfo>* Info;
		Info = &VMExpressionMap.Add(ir_unop_bit_not);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_not, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_logic_not);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::logic_not, glsl_type::bool_type, glsl_type::bool_type));
		Info = &VMExpressionMap.Add(ir_unop_neg);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::neg, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::negi, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_abs);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::abs, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::absi, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_sign);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::sign, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::signi, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_rcp);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::rcp, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_rsq);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::rsq, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_sqrt);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::sqrt, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_exp);      /**< Log base e on gentype */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::exp, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_log);	     /**< Natural log on gentype */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::log, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_exp2);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::exp2, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_log2);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::log2, glsl_type::float_type, glsl_type::float_type));

		Info = &VMExpressionMap.Add(ir_unop_f2i);      /**< Float-to-integer conversion. */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::f2i, glsl_type::int_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_i2f);      /**< Integer-to-float conversion. */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::i2f, glsl_type::float_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_f2b);      /**< Float-to-boolean conversion */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::f2b, glsl_type::bool_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_b2f);      /**< Boolean-to-float conversion */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::b2f, glsl_type::float_type, glsl_type::bool_type));
		Info = &VMExpressionMap.Add(ir_unop_i2b);      /**< int-to-boolean conversion */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::i2b, glsl_type::bool_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_unop_b2i);      /**< Boolean-to-int conversion */
		Info->Add(FVMExpresssionInfo(EVectorVMOp::b2i, glsl_type::int_type, glsl_type::bool_type));
		// 		Info = &VMExpressionMap.Add(ir_unop_b2u);
		// 		Info = &VMExpressionMap.Add(ir_unop_u2b);
		// 		Info = &VMExpressionMap.Add(ir_unop_f2u);
		// 		Info = &VMExpressionMap.Add(ir_unop_u2f);      /**< Unsigned-to-float conversion. */
		// 		Info = &VMExpressionMap.Add(ir_unop_i2u);      /**< Integer-to-unsigned conversion. */
		// 		Info = &VMExpressionMap.Add(ir_unop_u2i);      /**< Unsigned-to-integer conversion. */
		// 		Info = &VMExpressionMap.Add(ir_unop_h2i);
		// 		Info = &VMExpressionMap.Add(ir_unop_i2h);
		// 		Info = &VMExpressionMap.Add(ir_unop_h2f);
		// 		Info = &VMExpressionMap.Add(ir_unop_f2h);
		// 		Info = &VMExpressionMap.Add(ir_unop_h2b);
		// 		Info = &VMExpressionMap.Add(ir_unop_b2h);
		// 		Info = &VMExpressionMap.Add(ir_unop_h2u);
		//		Info = &VMExpressionMap.Add(ir_unop_u2h);

		//		Info = &VMExpressionMap.Add(ir_unop_transpose);

		//		Info = &VMExpressionMap.Add(ir_unop_any);
		//		Info = &VMExpressionMap.Add(ir_unop_all);

		/**
		* \name Unary floating-point rounding operations.
		*/
		/*@{*/
		Info = &VMExpressionMap.Add(ir_unop_trunc);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::trunc, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_ceil);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::ceil, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_floor);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::floor, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_fract);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::frac, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_round);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::round, glsl_type::float_type, glsl_type::float_type));
		/*@}*/

		/**
		* \name Trigonometric operations.
		*/
		/*@{*/
		Info = &VMExpressionMap.Add(ir_unop_sin);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::sin, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_cos);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cos, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_tan);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::tan, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_asin);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::asin, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_acos);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::acos, glsl_type::float_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_atan);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::atan, glsl_type::float_type, glsl_type::float_type));
		// 		Info = &VMExpressionMap.Add(ir_unop_sinh);//TODO;
		// 		Info = &VMExpressionMap.Add(ir_unop_cosh);
		// 		Info = &VMExpressionMap.Add(ir_unop_tanh);
		/*@}*/

		//		Info = &VMExpressionMap.Add(ir_unop_normalize);// Normalize isn't a valid single VM op as it requires cross talk between the scalar components of different instances.
		/*@}*/

		/**
		* \name Partial derivatives.
		*/
		/*@{*/
		//Info = &VMExpressionMap.Add(ir_unop_dFdx);
		//Info = &VMExpressionMap.Add(ir_unop_dFdy);
		/*@}*/

		//Info = &VMExpressionMap.Add(ir_unop_isnan);
		//Info = &VMExpressionMap.Add(ir_unop_isinf);

		/**
		* bit pattern casting operations
		*/
		// 		Info = &VMExpressionMap.Add(ir_unop_fasu);//TODO?
		Info = &VMExpressionMap.Add(ir_unop_fasi);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::fasi, glsl_type::int_type, glsl_type::float_type));
		Info = &VMExpressionMap.Add(ir_unop_iasf);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::iasf, glsl_type::float_type, glsl_type::int_type));
		// 		Info = &VMExpressionMap.Add(ir_unop_uasf);

		/**
		* Integer SM5 operations
		*/
		// 		Info = &VMExpressionMap.Add(ir_unop_bitreverse);
		// 		Info = &VMExpressionMap.Add(ir_unop_bitcount);
		// 		Info = &VMExpressionMap.Add(ir_unop_msb);
		// 		Info = &VMExpressionMap.Add(ir_unop_lsb);

		Info = &VMExpressionMap.Add(ir_unop_noise);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::noise, glsl_type::float_type, glsl_type::float_type));

		/**
		* A sentinel marking the last of the unary operations.
		*/
		Info = &VMExpressionMap.Add(ir_binop_add);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::add, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::addi, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_sub);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::sub, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::subi, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_mul);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::mul, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::muli, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_div);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::div, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::divi, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
	

		/**
		* Takes one of two combinations of arguments:
		*
		Info = &VMExpressionMap.Add(* - mod(vecN); vecN)
		Info = &VMExpressionMap.Add(* - mod(vecN); float)
		*
		* Does not take integer types.
		*/
		// 		Info = &VMExpressionMap.Add(ir_binop_mod);//TODO:
		// 		Info = &VMExpressionMap.Add(ir_binop_modf);

		Info = &VMExpressionMap.Add(ir_binop_step);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::step, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));

		/**
		* \name Binary comparison operators which return a boolean vector.
		* The type of both operands must be equal.
		*/
		/*@{*/
		Info = &VMExpressionMap.Add(ir_binop_less);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmplt, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmplti, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_greater);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpgt, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpgti, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_lequal);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmple, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmplei, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_gequal);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpge, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpgei, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_equal);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpeq, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpeqi, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpeqi, glsl_type::bool_type, glsl_type::bool_type, glsl_type::bool_type));
		Info = &VMExpressionMap.Add(ir_binop_nequal);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpneq, glsl_type::bool_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpneqi, glsl_type::bool_type, glsl_type::int_type, glsl_type::int_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::cmpneqi, glsl_type::bool_type, glsl_type::bool_type, glsl_type::bool_type));

		/**
		* Returns single boolean for whether all components of operands[0]
		* equal the components of operands[1].
		*/
		//Info = &VMExpressionMap.Add(ir_binop_all_equal);
		/**
		* Returns single boolean for whether any component of operands[0]
		* is not equal to the corresponding component of operands[1].
		*/
		//Info = &VMExpressionMap.Add(ir_binop_any_nequal);
		/*@}*/

		/**
		* \name Bit-wise binary operations.
		*/
		/*@{*/
		Info = &VMExpressionMap.Add(ir_binop_lshift);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_lshift, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_rshift);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_rshift, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_bit_and);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_and, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_bit_xor);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_xor, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_bit_or);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::bit_or, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		/*@}*/

		Info = &VMExpressionMap.Add(ir_binop_logic_and);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::logic_and, glsl_type::bool_type, glsl_type::bool_type, glsl_type::bool_type));
		Info = &VMExpressionMap.Add(ir_binop_logic_xor);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::logic_xor, glsl_type::bool_type, glsl_type::bool_type, glsl_type::bool_type));
		Info = &VMExpressionMap.Add(ir_binop_logic_or);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::logic_or, glsl_type::bool_type, glsl_type::bool_type, glsl_type::bool_type));

		//Info = &VMExpressionMap.Add(ir_binop_dot);
		//Info = &VMExpressionMap.Add(ir_binop_cross);
		Info = &VMExpressionMap.Add(ir_binop_min);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::min, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::mini, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));
		Info = &VMExpressionMap.Add(ir_binop_max);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::max, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::maxi, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));

		Info = &VMExpressionMap.Add(ir_binop_atan2);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::atan2, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));

		Info = &VMExpressionMap.Add(ir_binop_pow);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::pow, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));

		Info = &VMExpressionMap.Add(ir_ternop_lerp);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::lerp, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));

		Info = &VMExpressionMap.Add(ir_ternop_fma);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::mad, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));

		//Info = &VMExpressionMap.Add(ir_ternop_smoothstep);TODO: Smoothstep
		Info = &VMExpressionMap.Add(ir_ternop_clamp);
		Info->Add(FVMExpresssionInfo(EVectorVMOp::clamp, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type, glsl_type::float_type));
		Info->Add(FVMExpresssionInfo(EVectorVMOp::clampi, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type, glsl_type::int_type));

		//Info = &VMExpressionMap.Add(ir_quadop_vector);
	}
}

EVectorVMOp get_special_vm_opcode(ir_function_signature* signature)
{
	EVectorVMOp VVMOpCode = EVectorVMOp::done;
	if (strcmp(signature->function_name(), "rand") == 0)
	{
		unsigned num_operands = 0;
		foreach_iter(exec_list_iterator, iter, signature->parameters)
		{
			ir_variable *const param = (ir_variable *)iter.get();
			check(param->type->is_scalar());
			switch (param->type->base_type)
			{
			case GLSL_TYPE_FLOAT: VVMOpCode = EVectorVMOp::random; break;
			case GLSL_TYPE_INT: VVMOpCode = EVectorVMOp::randomi; break;
			//case GLSL_TYPE_BOOL: v->constant_table_size_bytes += sizeof(int32); break;
			default: check(0);
			}
			++num_operands;
		}
		check(num_operands == 1);
	}
	else if (strcmp(signature->function_name(), "Modulo") == 0)
	{
		return EVectorVMOp::fmod;
	}
	else if (strcmp(signature->function_name(), "select") == 0)
	{
		return EVectorVMOp::select;
	}
	else if (strcmp(signature->function_name(), "noise") == 0)
	{
		return EVectorVMOp::noise;
	}
	else if (strncmp(signature->function_name(), "InputDataNoadvanceFloat", strlen("InputDataNoadvanceFloat")) == 0)
	{
		return EVectorVMOp::inputdata_noadvance_float;
	}
	else if (strncmp(signature->function_name(), "InputDataNoadvanceInt", strlen("InputDataNoadvanceInt")) == 0)
	{
		return EVectorVMOp::inputdata_noadvance_int32;
	}
	else if (strncmp(signature->function_name(), "InputDataNoadvanceBool", strlen("InputDataNoadvanceBool")) == 0)
	{
		return EVectorVMOp::inputdata_noadvance_int32;
	}
	else if (strncmp(signature->function_name(), "InputDataNoadvanceHalf", strlen("InputDataNoadvanceHalf")) == 0)
	{
		return EVectorVMOp::inputdata_noadvance_half;
	}
	else if (strncmp(signature->function_name(), "InputDataHalf", strlen("InputDataHalf")) == 0)
	{
		return EVectorVMOp::inputdata_half;
	}
	else if (strncmp(signature->function_name(), "OutputDataHalf", strlen("OutputDataHalf")) == 0)
	{
		return EVectorVMOp::outputdata_half;
	}
	else if (strncmp(signature->function_name(), "InputDataInt", strlen("InputDataInt")) == 0)
	{
		return EVectorVMOp::inputdata_int32;
	}
	else if (strncmp(signature->function_name(), "OutputDataInt", strlen("OutputDataInt")) == 0)
	{
		return EVectorVMOp::outputdata_int32;
	}
	else if (strncmp(signature->function_name(), "InputDataBool", strlen("InputDataBool")) == 0)
	{
		return EVectorVMOp::inputdata_int32;
	}
	else if (strncmp(signature->function_name(), "OutputDataBool", strlen("OutputDataBool")) == 0)
	{
		return EVectorVMOp::outputdata_int32;
	}
	else if (strncmp(signature->function_name(), "InputDataFloat", strlen("InputDataFloat")) == 0)
	{
		return EVectorVMOp::inputdata_float;
	}
	else if (strncmp(signature->function_name(), "OutputDataFloat", strlen("OutputDataFloat")) == 0)
	{
		return EVectorVMOp::outputdata_float;
	}
	else if (strcmp(signature->function_name(), "AcquireIndex") == 0)
	{
		return EVectorVMOp::acquireindex;
	}
	else if (strcmp(signature->function_name(), "AcquireID") == 0)
	{
		return EVectorVMOp::acquire_id;
	}
	else if (strcmp(signature->function_name(), "UpdateID") == 0)
	{
		return EVectorVMOp::update_id;
	}
	else if (strcmp(signature->function_name(), "ExecIndex") == 0)
	{
		return EVectorVMOp::exec_index;
	}
	else if (strcmp(signature->function_name(), "EnterStatScope") == 0)
	{
		return EVectorVMOp::enter_stat_scope;
	}
	else if (strcmp(signature->function_name(), "ExitStatScope") == 0)
	{
		return EVectorVMOp::exit_stat_scope;
	}
	else if (signature->body.get_head() == nullptr)
	{
		return EVectorVMOp::external_func_call;
	}
	check(VVMOpCode != EVectorVMOp::done);
	return VVMOpCode;
}

/** Tree structure for variable infos allowing easier access in some cases. */
struct variable_info_node
{
	struct variable_info* owner;
	const char* name;
	const glsl_type* type;

	/** Array of child components. Will be 0 and null for scalars/leaves.*/
	unsigned num_children;
	variable_info_node** children;

	/** used only for scalar/leaf nodes */

	/** index of assignment this component was last read at.*/
	unsigned last_read;
	/** offset of the data for this component. */
	unsigned offset;

	bool is_scalar()
	{
		return type->is_scalar();
	}
};

typedef void(*node_func)(variable_info_node*, void* user_ptr);

void for_each_component(variable_info_node* node, node_func func, bool scalars_only, void* user_ptr)
{
	if (!scalars_only || node->is_scalar())
		func(node, user_ptr);

	for (unsigned i = 0; i < node->num_children; ++i)
	{
		for_each_component(node->children[i], func, scalars_only, user_ptr);
	}
}

struct variable_info
{
	ir_instruction *ir;

	EVectorVMOperandLocation location;

	variable_info_node* root;

	void Init(void* mem_ctx, ir_instruction* in_ir, EVectorVMOperandLocation in_location)
	{
		ir = in_ir;
		location = in_location;

		const char* irName;
		const glsl_type* irType;
		if (ir_constant* constant = ir->as_constant())
		{
			irName = "constant";
			irType = constant->type;
		}
		else if (ir_variable* var = ir->as_variable())
		{
			irName = var->name;
			irType = var->type;
		}
		else
		{
			check(0);//unhandled ir type.
			return;
		}

		variable_info* owner = this;
		TFunction<void(const char*, const glsl_type*, variable_info_node**)> build_component_info_tree;

		build_component_info_tree = [&](const char* name, const glsl_type* type, variable_info_node** curr)
		{
			variable_info_node* new_node = ralloc(mem_ctx, variable_info_node);
			new_node->owner = owner;
			new_node->name = name;
			new_node->type = type;
			new_node->num_children = 0;
			new_node->children = nullptr;
			new_node->last_read = INDEX_NONE;
			new_node->offset = INDEX_NONE;
			*curr = new_node;

			const glsl_type* base_type = type->get_base_type();
			if (type->is_scalar())
			{
				new_node->num_children = 0;
			}
			else if (type->is_vector())
			{
				new_node->num_children = type->components();
				new_node->children = ralloc_array(mem_ctx, variable_info_node*, new_node->num_children);

				build_component_info_tree("x", base_type, &new_node->children[0]);
				build_component_info_tree("y", base_type, &new_node->children[1]);
				if (type->components() > 2)
				{
					build_component_info_tree("z", base_type, &new_node->children[2]);
				}
				if (type->components() > 3)
				{
					build_component_info_tree("w", base_type, &new_node->children[3]);
				}
			}
			else if (type->is_matrix())
			{
				const char* matrix_names[4] = { "Row0", "Row1", "Row2", "Row3" };

				new_node->num_children = type->vector_elements;
				new_node->children = ralloc_array(mem_ctx, variable_info_node*, new_node->num_children);

				unsigned rows = type->vector_elements;
				for (unsigned r = 0; r < rows; ++r)
				{
					build_component_info_tree(matrix_names[r], type->row_type(), &new_node->children[r]);
				}
			}
			else if (type->base_type == GLSL_TYPE_STRUCT)
			{
				new_node->num_children = type->length;
				new_node->children = ralloc_array(mem_ctx, variable_info_node*, new_node->num_children);

				for (unsigned member_idx = 0; member_idx < type->length; ++member_idx)
				{
					glsl_struct_field& struct_field = type->fields.structure[member_idx];
					build_component_info_tree(struct_field.name, struct_field.type, &new_node->children[member_idx]);
				}
			}
			else
			{
				//This type is currently unhandled.
				check(0);
			}
		};
		build_component_info_tree(irName, irType, &root);
	}
};

void emit_byte(unsigned char b, TArray<uint8>& bytecode)
{
	bytecode.Add(b);
}

void emit_short(unsigned short s, TArray<uint8>& bytecode)
{
	bytecode.Add(s & 255);
	bytecode.Add(s >> 8);
}

void emit_external_func_input(variable_info_node* input, TArray<uint8>& bytecode)
{
	check((input->offset & VVM_EXT_FUNC_INPUT_LOC_BIT) == 0);//Ensure the offset isn't too large.

	unsigned short offset = input->offset | (input->owner->location == EVectorVMOperandLocation::Constant ? 0 : VVM_EXT_FUNC_INPUT_LOC_BIT);
	bytecode.Add(offset & 255);
	bytecode.Add(offset >> 8);
}

struct op_base
{
	op_base()
		: op_code(EVectorVMOp::done)
	{}

	virtual ~op_base() {}

	EVectorVMOp op_code;

	virtual FString to_string() = 0;
	virtual void add_to_bytecode(TArray<uint8>& bytecode) = 0;
	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx) { return true; }
	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx) = 0;

	virtual struct op_standard* as_standard() { return nullptr; }
	virtual struct op_hook* as_hook() { return nullptr; }
	virtual struct op_input* as_input() { return nullptr; }
	virtual struct op_output* as_output() { return nullptr; }
	virtual struct op_index_acquire* as_index_acquire() { return nullptr; }
	virtual struct op_id_acquire* as_id_acquire() { return nullptr; }
	virtual struct op_id_update* as_id_update() { return nullptr; }
	virtual struct op_external_func* as_external_func() { return nullptr; }

	static void validate_component_offset(_mesa_glsl_parse_state* parse_state, variable_info_node* component, unsigned op_idx)
	{
		if (component->offset == INDEX_NONE && component->last_read >= (uint32)op_idx)
		{
			FString ErrorStr = FString::Printf(TEXT("Component %s of variable %s has no valid offset. Possibly uninitialized data being used."), ANSI_TO_TCHAR(component->name), ANSI_TO_TCHAR(component->owner->root->name));

			_mesa_glsl_error(parse_state, TCHAR_TO_ANSI(*ErrorStr));
		}

	}

	static bool finalize_component_temporary_allocation(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput& CompilationOutput, variable_info_node* component, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		check(component);

		if (component->last_read <= op_idx || component->last_read == INDEX_NONE)
		{
			//It's possible that some components can have invalid indices here. We let the calling operations deal with this as they see fit.
			return false;
		}

		if (ir_variable* var = component->owner->ir->as_variable())
		{
			if (var->mode == ir_var_temporary || var->mode == ir_var_auto)
			{
				if (component->offset == INDEX_NONE)
				{
					for (int32 j = 0; j < temp_registers.Num(); ++j)
					{
						if (temp_registers[j] < op_idx || temp_registers[j] == INDEX_NONE)
						{
							component->offset = j;
							temp_registers[j] = component->last_read;
							CompilationOutput.MaxTempRegistersUsed = FMath::Max((int32)j, CompilationOutput.MaxTempRegistersUsed);
#if VM_VERBOSE_LOGGING
							UE_LOGF(LogVVMBackend, Log, "OP:%d | Comp:%p allocated Reg: %d | Last Read: %d |", op_idx, component, j, component->last_read);
#endif
							break;
						}
					}

					//If we didn't find a free temp register to use, add a new one.
					if (component->offset == INDEX_NONE)
					{
						component->offset = temp_registers.Num();
						CompilationOutput.MaxTempRegistersUsed = FMath::Max(temp_registers.Num(), CompilationOutput.MaxTempRegistersUsed);
						temp_registers.Add(component->last_read);
#if VM_VERBOSE_LOGGING
						UE_LOGF(LogVVMBackend, Log, "OP:%d | Comp:%p allocated Reg: %d | Last Read: %d |", op_idx, component, temp_registers.Num() - 1, component->last_read);
#endif
					}
 
// 					if (component->offset == INDEX_NONE)
// 					{
// 						FString ErrorStr = FString::Printf(TEXT("Failed to allocate temporary register for component %s of variable %s."), ANSI_TO_TCHAR(component->name), ANSI_TO_TCHAR(component->owner->root->name));
// 						_mesa_glsl_error(parse_state, TCHAR_TO_ANSI(*ErrorStr));
// 					}
				}
			}
			else
			{
				FString ErrorStr = FString::Printf(TEXT("Attempted to allocate temporary register for invalid variable. Component %s of variable %s."), ANSI_TO_TCHAR(component->name), ANSI_TO_TCHAR(component->owner->root->name));
				_mesa_glsl_error(parse_state, TCHAR_TO_ANSI(*ErrorStr));
			}
		}
		else
		{
			_mesa_glsl_error(parse_state, "Attempted to allocate temporary register for null variable.");
		}
		return true;
	}

	const TCHAR* get_location_string(EVectorVMOperandLocation Loc) { return Loc == EVectorVMOperandLocation::Constant ? TEXT("C") : TEXT("R"); }
};

//Simple hook ops to mark the code and hook into other systems such as stats.
struct op_hook : public op_base
{
	op_hook() : source_component(nullptr)
	{
	}
	variable_info_node* source_component;

	virtual op_hook* as_hook() { return this; }

	virtual FString to_string()
	{
		TArray<FStringFormatArg> Args;
		Args.Reserve(3);

		static const TCHAR* FormatStrs[4] =
		{
			TEXT("{0}();\n"),
			TEXT("{0}({1}[{2}]);\n"),
		};

#if WITH_EDITOR
		Args.Add(VectorVM::GetOpName(op_code));
#else
		Args.Add(FString::Printf(TEXT("__OP__%d"), EnumToUnderlyingType(op_code)));
#endif

		if(source_component)
		{
			Args.Add(get_location_string(source_component->owner->location));
			Args.Add(source_component->offset);
		}

		return source_component ? FString::Format(FormatStrs[1], Args) : FString::Format(FormatStrs[0], Args);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		if (source_component)
		{
			emit_short(source_component->offset, bytecode);
		}
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		if (source_component)
		{
			validate_component_offset(parse_state, source_component, op_idx);
		}
	}
};

struct op_standard : public op_base
{
	op_standard()
		: dest_component(nullptr)
		, num_operands(0)
	{
		for (unsigned i = 0; i < 3; ++i)
		{
			source_components[i] = nullptr;
		}
	}

	virtual op_standard* as_standard() { return this; }

	variable_info_node* dest_component;
	variable_info_node* source_components[3];
	unsigned num_operands;

	virtual FString to_string()
	{
		TArray<FStringFormatArg> Args;
		Args.Reserve(9);

		static const TCHAR* FormatStrs[4] =
		{
			TEXT("{1}[{2}] = {0};\n"),
			TEXT("{1}[{2}] = {0}({3}[{4}]);\n"),
			TEXT("{1}[{2}] = {0}({3}[{4}], {5}[{6}]);\n"),
			TEXT("{1}[{2}] = {0}({3}[{4}], {5}[{6}], {7}[{8}]);\n"),
		};
		check(dest_component);

#if WITH_EDITOR
		Args.Add(VectorVM::GetOpName(op_code));
#else
		Args.Add(FString::Printf(TEXT("__OP__%d"), EnumToUnderlyingType(op_code)));
#endif

		Args.Add(get_location_string(dest_component->owner->location));
		Args.Add(dest_component->offset);

		for (unsigned operand_idx = 0; operand_idx < num_operands; ++operand_idx)
		{
			Args.Add(get_location_string(source_components[operand_idx]->owner->location));
			Args.Add(source_components[operand_idx]->offset);
		}

		return FString::Format(FormatStrs[num_operands], Args);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);

		if (num_operands > 0)
		{
			EVectorVMOperandLocation operand_locs[3] = { EVectorVMOperandLocation::Register, EVectorVMOperandLocation::Register, EVectorVMOperandLocation::Register };
			for (unsigned operand_idx = 0; operand_idx < num_operands; ++operand_idx)
			{
				operand_locs[operand_idx] = source_components[operand_idx]->owner->location;
			}
			emit_byte(VectorVM::CreateSrcOperandMask(operand_locs[0], operand_locs[1], operand_locs[2]), bytecode);
		}
		for (unsigned operand_idx = 0; operand_idx < num_operands; ++operand_idx)
		{
			emit_short(source_components[operand_idx]->offset, bytecode);
		}

		check(dest_component);
		emit_short(dest_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		return finalize_component_temporary_allocation(parse_state, CompilationOutput, dest_component, temp_registers, op_idx);
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		for (unsigned operand_idx = 0; operand_idx < num_operands; ++operand_idx)
		{
			validate_component_offset(parse_state, source_components[operand_idx], op_idx);
		}

		validate_component_offset(parse_state, dest_component, op_idx);
	}
};

struct op_input : public op_base
{
	op_input()
		: dataset_idx(INDEX_NONE)
		, register_idx(INDEX_NONE)
		, dest_component(nullptr)
		, instance_idx_component(nullptr)
	{}
	virtual op_input* as_input() { return this; }
	unsigned dataset_idx;
	unsigned register_idx;
	variable_info_node* dest_component;
	variable_info_node* instance_idx_component;
	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputData(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputData(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(dest_component);
		//if(instance_idx_component)//TODO: Handle the case with an explicit instance index. Probably want a separate VMOp for this 
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		emit_short(dataset_idx, bytecode);
		emit_short(register_idx, bytecode);
		//Destination is a temp register which we emit as usual.
		check(dest_component);
		emit_short(dest_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		return finalize_component_temporary_allocation(parse_state, CompilationOutput, dest_component, temp_registers, op_idx);
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, dest_component, op_idx);
	}
};

struct op_input_half : public op_input
{
	op_input_half(): op_input(){}

	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataHalf(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataHalf(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};


struct op_input_float : public op_input
{
	op_input_float() : op_input() {}

	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataFloat(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataFloat(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};

struct op_input_int : public op_input
{
	op_input_int() : op_input() {}

	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataInt(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataInt(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};

// TODO: make this a proper explicitly indexing op
struct op_input_noadvance : public op_input
{
	op_input_noadvance()
	{}
	virtual op_input* as_input() { return this; }
	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvance(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvance(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(dest_component);
		//if(instance_idx_component)//TODO: Handle the case with an explicit instance index. Probably want a separate VMOp for this 
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		emit_short(dataset_idx, bytecode);
		emit_short(register_idx, bytecode);
		//Destination is a temp register which we emit as usual.
		check(dest_component);
		emit_short(dest_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		return finalize_component_temporary_allocation(parse_state, CompilationOutput, dest_component, temp_registers, op_idx);
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, dest_component, op_idx);
	}
};

struct op_input_noadvance_int : public op_input_noadvance
{
	op_input_noadvance_int() : op_input_noadvance() {}
	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceInt(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceInt(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};

struct op_input_noadvance_float : public op_input_noadvance
{
	op_input_noadvance_float() : op_input_noadvance() {}
	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceFloat(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceFloat(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};

struct op_input_noadvance_half : public op_input_noadvance
{
	op_input_noadvance_half() : op_input_noadvance() {}
	virtual FString to_string()
	{
		if (instance_idx_component)
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceHalf(%d, %d, [%s][%d]);\n"),
				dest_component->offset,
				dataset_idx,
				register_idx,
				get_location_string(instance_idx_component->owner->location),
				instance_idx_component->offset);
		}
		else
		{
			return FString::Printf(TEXT("[R][%d] = InputDataNoadvanceHalf(%d, %d);\n"), dest_component->offset, dataset_idx, register_idx);
		}
	}
};

struct op_output : public op_base
{
	op_output()
		: register_idx(INDEX_NONE)
		, instance_idx_component(nullptr)
		, value_component(nullptr)
	{}
	virtual op_output* as_output() { return this; }
	unsigned register_idx, dataset_index;
	variable_info_node* instance_idx_component;
	variable_info_node* value_component;

	virtual FString to_string()
	{
		check(instance_idx_component);
		check(value_component);

		return FString::Printf(TEXT("OutputData(%d, %d, [%s][%d], [%s][%d]);\n"),
			dataset_index,
			register_idx,
			get_location_string(instance_idx_component->owner->location),
			instance_idx_component->offset,
			get_location_string(value_component->owner->location),
			value_component->offset);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(instance_idx_component);
		check(value_component);
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		//value can be a constant or a register so have to emit a location byte.
		emit_byte(VectorVM::CreateSrcOperandMask(value_component->owner->location), bytecode);
		emit_short(dataset_index, bytecode);
		emit_short(instance_idx_component->offset, bytecode);
		emit_short(value_component->offset, bytecode);
		emit_short(register_idx, bytecode);
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, value_component, op_idx);
	}
};

struct op_output_half : public op_output
{
	op_output_half(): op_output(){}

	virtual FString to_string()
	{
		check(instance_idx_component);
		check(value_component);

		return FString::Printf(TEXT("OutputDataHalf(%d, %d, [%s][%d], [%s][%d]);\n"),
			dataset_index,
			register_idx,
			get_location_string(instance_idx_component->owner->location),
			instance_idx_component->offset,
			get_location_string(value_component->owner->location),
			value_component->offset);
	}
};

struct op_output_float : public op_output
{
	op_output_float() : op_output() {}

	virtual FString to_string()
	{
		check(instance_idx_component);
		check(value_component);

		return FString::Printf(TEXT("OutputDataFloat(%d, %d, [%s][%d], [%s][%d]);\n"),
			dataset_index,
			register_idx,
			get_location_string(instance_idx_component->owner->location),
			instance_idx_component->offset,
			get_location_string(value_component->owner->location),
			value_component->offset);
	}
};

struct op_output_int : public op_output
{
	op_output_int() : op_output() {}

	virtual FString to_string()
	{
		check(instance_idx_component);
		check(value_component);

		return FString::Printf(TEXT("OutputDataInt(%d, %d, [%s][%d], [%s][%d]);\n"),
			dataset_index,
			register_idx,
			get_location_string(instance_idx_component->owner->location),
			instance_idx_component->offset,
			get_location_string(value_component->owner->location),
			value_component->offset);
	}
};

struct op_index_acquire : public op_base
{
	op_index_acquire()
		: dest_component(nullptr)
		, dataset_idx(INDEX_NONE)
		, valid_component(nullptr)
	{}

	virtual op_index_acquire* as_index_acquire() { return this; }

	variable_info_node* dest_component;
	unsigned dataset_idx;
	variable_info_node* valid_component;

	virtual FString to_string()
	{
		check(valid_component);

		return FString::Printf(TEXT("[R][%d] = AcquireIndex(%d, [%s][%d]);\n"),
			dest_component->offset,
			dataset_idx,
			get_location_string(valid_component->owner->location),
			valid_component->offset);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(valid_component);
		check(dest_component);
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		emit_byte(VectorVM::CreateSrcOperandMask(valid_component->owner->location), bytecode);
		emit_short(dataset_idx, bytecode);
		emit_short(valid_component->offset, bytecode);
		emit_short(dest_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		return finalize_component_temporary_allocation(parse_state, CompilationOutput, dest_component, temp_registers, op_idx);
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, dest_component, op_idx);
	}
};

struct op_id_acquire : public op_base
{
	op_id_acquire()
		: dataset_idx(INDEX_NONE)
		, id_index_component(nullptr)
		, id_tag_component(nullptr)
	{}

	virtual op_id_acquire* as_id_acquire() { return this; }

	unsigned dataset_idx;
	variable_info_node* id_index_component;
	variable_info_node* id_tag_component;

	virtual FString to_string()
	{
		check(id_index_component);
		check(id_tag_component);

		return FString::Printf(TEXT("AcquireID(%d, [R][%d], [R][%d]);\n"), dataset_idx, id_index_component->offset, id_tag_component->offset);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(id_index_component);
		check(id_tag_component);
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		emit_short(dataset_idx, bytecode);
		emit_short(id_index_component->offset, bytecode);
		emit_short(id_tag_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		if (!finalize_component_temporary_allocation(parse_state, CompilationOutput, id_index_component, temp_registers, op_idx))
		{
			_mesa_glsl_error(parse_state, "Failed to allocate temporary registers");
			return false;
		}
		if (!finalize_component_temporary_allocation(parse_state, CompilationOutput, id_tag_component, temp_registers, op_idx))
		{
			_mesa_glsl_error(parse_state, "Failed to allocate temporary registers");
			return false;
		}
		return true;
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, id_index_component, op_idx);
		validate_component_offset(parse_state, id_tag_component, op_idx);
	}
};

struct op_id_update : public op_base
{
	op_id_update()
		: dataset_idx(INDEX_NONE)
		, id_component(nullptr)
		, index_component(nullptr)
	{}

	virtual op_id_update* as_id_update() { return this; }

	unsigned dataset_idx;
	variable_info_node* id_component;
	variable_info_node* index_component;

	virtual FString to_string()
	{
		check(id_component);
		check(index_component);

		return FString::Printf(TEXT("UpdateID(%d, [%d], [%d]);\n"),
			dataset_idx,
			id_component->offset,
			index_component->offset);
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		check(id_component);
		check(index_component);
		check(op_code != EVectorVMOp::done);
		emit_byte((uint8)op_code, bytecode);
		emit_short(dataset_idx, bytecode);
		emit_short(id_component->offset, bytecode);
		emit_short(index_component->offset, bytecode);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		return true;
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		validate_component_offset(parse_state, id_component, op_idx);
		validate_component_offset(parse_state, index_component, op_idx);
	}
};

struct op_external_func : public op_base
{
	virtual op_external_func* as_external_func() { return this; }

	op_external_func()
		: function_index(INDEX_NONE)
	{}

	TArray<variable_info_node*> inputs;
	TArray<variable_info_node*> outputs;
	unsigned function_index;
	ir_function_signature* sig;
	
	virtual FString to_string() 
	{ 
		FString Str = FString::Printf(TEXT("%s[func%d]("), ANSI_TO_TCHAR(sig->function_name()), function_index);
		int32 param_idx = 0;
		FString ComponentStr;
		for (int32 i = 0; i < inputs.Num(); ++i, ++param_idx)
		{
			variable_info_node* input = inputs[i];
			if (param_idx > 0)
			{
				Str += TEXT(", ");
			}
			check(input->is_scalar());
			ComponentStr.Empty(64);
			ComponentStr = LexToString(input->offset);

			Str += FString::Printf(TEXT("%s[%s]"), get_location_string(input->owner->location), *ComponentStr);
		}
		
		for (int32 i = 0; i < outputs.Num(); ++i, ++param_idx)
		{
			variable_info_node* output = outputs[i];
			if (param_idx > 0)
			{
				Str += TEXT(", ");
			}

			ComponentStr.Empty(64);
			check(output->is_scalar());

			if (output->offset == INDEX_NONE)
			{
				Str += TEXT("Unused");
			}
			else
			{
				ComponentStr = LexToString(output->offset);
				Str += FString::Printf(TEXT("%s[%s]"), get_location_string(output->owner->location), *ComponentStr);
			}
		}

		return Str += TEXT(");\n");
	}

	bool should_cull_function(unsigned int op_idx)
	{
		FString FuncName = FString::Printf(TEXT("%s"), ANSI_TO_TCHAR(sig->function_name()));
		bool bPotentiallyHasSideEffects = FuncName.Contains(TEXT("_UEImpureCall"));
		if (bPotentiallyHasSideEffects)
		{
			return false;
		}

		//If the outputs of this call are never used after the current op then we cull this function.
		bool func_is_used = false;
		for (variable_info_node* output : outputs)
		{
			if (output->last_read > op_idx && output->last_read != INDEX_NONE)
			{
				func_is_used = true;
				break;
			}
		}

		return !func_is_used;
	}

	virtual void add_to_bytecode(TArray<uint8>& bytecode)
	{
		if (function_index == -1)
		{
			return;
		}

		emit_byte((uint8)EVectorVMOp::external_func_call, bytecode);

		check(function_index < std::numeric_limits<uint16>::max());
		emit_short(function_index, bytecode);

		for (variable_info_node* input : inputs)
		{
			emit_external_func_input(input, bytecode);
		}

		for (variable_info_node* output : outputs)
		{
			emit_short(output->offset, bytecode);
		}
	}

	struct finalize_temp_register_ctx
	{
		_mesa_glsl_parse_state* parse_state;
		FVectorVMCompilationOutput* CompilationOutput;
		TArray<uint32>* temp_registers;
		unsigned op_idx;
		bool success;
	};

	static void finalize_comp(variable_info_node* node, void* user_ptr)
	{
		finalize_temp_register_ctx* ctx = (finalize_temp_register_ctx*)user_ptr;
		check(node->is_scalar());

		//Some outputs for an external function can have invalid locations. If the thing has *any* valid outputs then it's considered a success.
		ctx->success |= finalize_component_temporary_allocation(ctx->parse_state, *ctx->CompilationOutput, node, *ctx->temp_registers, ctx->op_idx);
	}

	virtual bool finalize(_mesa_glsl_parse_state* parse_state, FVectorVMCompilationOutput&CompilationOutput, TArray<uint32>& temp_registers, unsigned op_idx)
	{
		//don't allocate any temp registers if we're culling the function.
		function_index = -1;
		if (should_cull_function(op_idx))
		{
			return false;
		}

		//We also add this call to the compiler output here as we're now sure it's been called.
		function_index = CompilationOutput.CalledVMFunctionTable.AddDefaulted();

		// we're also reserving one slot for -1 as an invalid index
		constexpr unsigned int MaxExternalFunctionCount = std::numeric_limits<uint16>::max() - 1;
		if (function_index >= MaxExternalFunctionCount)
		{
			_mesa_glsl_error(parse_state, "Exceeded maximum number (%d) of available external function calls.", MaxExternalFunctionCount);
			return false;
		}

		check(function_index != INDEX_NONE);
		FCalledVMFunction& CalledFunc = CompilationOutput.CalledVMFunctionTable[function_index];
		CalledFunc.Name = sig->function_name();
		CalledFunc.NumOutputs = outputs.Num();
		for (variable_info_node* input : inputs)
		{
			CalledFunc.InputParamLocations.Add(input->owner->location == EVectorVMOperandLocation::Constant);
		}

		finalize_temp_register_ctx ctx;
		ctx.parse_state = parse_state;
		ctx.CompilationOutput = &CompilationOutput;
		ctx.temp_registers = &temp_registers;
		ctx.op_idx = op_idx;
		ctx.success = true;
		for (variable_info_node* output : outputs)
		{
			for_each_component(output, &finalize_comp, true, &ctx);
		}
		if (!ctx.success)
		{
			_mesa_glsl_error(parse_state, "Failed to allocate temporary registers for external function.");
		}
		return ctx.success;
	}

	virtual void validate(_mesa_glsl_parse_state* parse_state, unsigned op_idx)override
	{
		if (function_index == -1)
		{
			return;
		}

		for (variable_info_node* input : inputs)
		{
			validate_component_offset(parse_state, input, op_idx);
		}

		//Output offsets are allowed to be INDEX_NONE if they are unused.
// 		for (variable_info_node* output : outputs)
// 		{
// 			validate_component_offset(parse_state, output, op_idx);
// 		}
	}
};

//////////////////////////////////////////////////////////////////////////

/**
There are four stages to this visitor.
1. Visits to all ir_variable or ir_constant generates a var_info and var_info_node tree that describes various information relating to it's location and usage in the vm.
2. Visits to all dereference chain ir traverse find correct var_info and traverse the var_info_tree to find the correct scalar value node to be used in expressions, assinments and calls.
3. Visits to assignments, calls and expressions gather operand dereferences and generate an ordered list of operations containing the correct values from the var_info tree.
4. These ordered operations are iterated over to produce the final VM bytecode.
*/
class ir_gen_vvm_visitor : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state *parse_state;
	void *mem_ctx;

	/** Helper class allowing us to feed information about constants back into the compilers representation of constants. This can be removed when "internal" constants are handled differently. */
	FVectorVMCompilationOutput& CompilationOutput;

	/** contains a var_info for each variable and constant seen. Each having a tree describing the whole variable and data locations. */
	hash_table* var_info_table;

	TArray<variable_info*> param_vars;
	unsigned num_input_components;
	unsigned num_output_components;
	unsigned constant_table_size_bytes;

	/** ordered list of operations. Used to generate the final vm bytecode. */
	TArray<op_base*> ordered_ops;

	unsigned dest_component;

	/** Used when building ordered_ops. Keeps track of the current operands info node so we can generate it's op code and data location. */
	variable_info_node* curr_node;

	//currently handling an output param?
	bool is_output_param;

	TArray<ir_constant*> seen_constants;

	explicit ir_gen_vvm_visitor(_mesa_glsl_parse_state *in_parse_state, FVectorVMCompilationOutput& InCompilationOutput)
		: parse_state(in_parse_state)
		, mem_ctx(ralloc_context(0))
		, CompilationOutput(InCompilationOutput)
		, var_info_table(hash_table_ctor(32, hash_table_pointer_hash, hash_table_pointer_compare))
		, num_input_components(0)
		, num_output_components(0)
		, constant_table_size_bytes(0)
		, dest_component(0)
		, curr_node(nullptr)
		, is_output_param(false)
	{
	}

	virtual ~ir_gen_vvm_visitor()
	{
		for (int32 i = 0; i < ordered_ops.Num(); ++i)
		{
			check(ordered_ops[i]);
			delete ordered_ops[i];
		}

		hash_table_dtor(var_info_table);
		ralloc_free(mem_ctx);
	}

	/** assigns an input register to a variable_info_node. */
	static void assign_input_loc(variable_info_node* node, void* user_ptr)
	{
		ir_gen_vvm_visitor* v = (ir_gen_vvm_visitor*)user_ptr;
		check(node->is_scalar());
		node->offset = v->num_input_components++;
	}

	/** assigns an output register to a variable_info_node. */
	static void assign_output_loc(variable_info_node* node, void* user_ptr)
	{
		ir_gen_vvm_visitor* v = (ir_gen_vvm_visitor*)user_ptr;
		check(node->is_scalar());
		node->offset = v->num_output_components++;
	}

	/** assigns an uniform buffer address to a variable_info_node. */
	static void assign_uniform_loc(variable_info_node* node, void* user_ptr)
	{
		ir_gen_vvm_visitor* v = (ir_gen_vvm_visitor*)user_ptr;
		check(node->is_scalar());

		v->param_vars.AddUnique(node->owner);
		node->offset = v->constant_table_size_bytes;
		switch (node->type->base_type)
		{
		case GLSL_TYPE_FLOAT: v->constant_table_size_bytes += sizeof(float); break;
		case GLSL_TYPE_INT: v->constant_table_size_bytes += sizeof(int32); break;
		case GLSL_TYPE_BOOL: v->constant_table_size_bytes += sizeof(int32); break;
		default: check(0);
		}
	}

	/** We visit each variable, decomposing it into a variable_info tree and assigning it's register or buffer locations. */
	virtual ir_visitor_status visit(ir_variable *var)
	{
		switch (var->mode)
		{
		case ir_var_auto:
		case ir_var_temporary:
		{
			variable_info *varinfo = (variable_info *)hash_table_find(var_info_table, var);
			check(varinfo == nullptr);
			varinfo = ralloc(mem_ctx, variable_info);
			hash_table_insert(var_info_table, varinfo, var);

			varinfo->Init(mem_ctx, var, EVectorVMOperandLocation::Register);
		}
		break;
		case ir_var_in:
		{
			//Input and output locations are now handled by special case operations.
			// 			variable_info *varinfo = (variable_info*)hash_table_find(var_info_table, var);
			// 			check(varinfo == nullptr);
			// 			varinfo = ralloc(mem_ctx, variable_info);
			// 			hash_table_insert(var_info_table, varinfo, var);
			// 
			// 			varinfo->Init(mem_ctx, var, EVectorVMOperandLocation::Register);
			// 
			// 			for_each_component(varinfo->root, &assign_input_loc, true, this);
		}
		break;
		case ir_var_out:
		{
			//Input and output locations are now handled by special case operations.
			// 			variable_info *varinfo = (variable_info*)hash_table_find(var_info_table, var);
			// 			check(varinfo == nullptr);
			// 			varinfo = ralloc(mem_ctx, variable_info);
			// 			hash_table_insert(var_info_table, varinfo, var);
			// 
			// 			varinfo->Init(mem_ctx, var, EVectorVMOperandLocation::Register);
			// 
			// 			for_each_component(varinfo->root, &assign_output_loc, true, this);
		}
		break;
		case ir_var_uniform:
		{
			variable_info *varinfo = (variable_info*)hash_table_find(var_info_table, var);
			
			//We can possibly hit the same variables twice here as we do a pre pass to ensure we visit them in the correct order.
			if (varinfo == nullptr)
			{
				varinfo = ralloc(mem_ctx, variable_info);
				hash_table_insert(var_info_table, varinfo, var);

				varinfo->Init(mem_ctx, var, EVectorVMOperandLocation::Constant);

				for_each_component(varinfo->root, &assign_uniform_loc, true, this);
			}
		};
		break;
		}
		return visit_continue;
	}

	/** When visiting constants, we add them to the compiler output constant set and generate a var_info tree for them, adding them to the uniform buffer. */
	virtual ir_visitor_status visit(ir_constant *constant)
	{
		variable_info *varinfo = (variable_info*)hash_table_find(var_info_table, constant);
		check(!in_assignee);
		check(constant->type->is_scalar());//We shouldn't ever have non scalar constants.
		check(curr_node == nullptr);
		check(varinfo == nullptr);
		check(!is_output_param);

		const glsl_type* type = constant->type;
		int32 same_const = seen_constants.IndexOfByPredicate(
			[&](ir_constant* other)
		{
			if (type == other->type)
			{
				switch (other->type->base_type)
				{
				case GLSL_TYPE_FLOAT: 
				{			
					float val = constant->value.f[0];
					float otherval = other->value.f[0];
					if ( FMath::IsFinite(val) && FMath::IsFinite(otherval) )
					{
						return val == otherval;
					}
					return FMath::IsFinite(val) == FMath::IsFinite(otherval) && FMath::IsNaN(val) == FMath::IsNaN(otherval);
				}
				case GLSL_TYPE_INT: return constant->value.i[0] == other->value.i[0];
				case GLSL_TYPE_BOOL: return constant->value.b[0] == other->value.b[0];
				default: check(0);
				};
			}
			return false;
		}
		);

		if (!is_supported_base_type(type))
		{
			_mesa_glsl_error(parse_state, "this base type is not supported in VVM currently: %s", type->name);
			return visit_stop;
		}

		if (same_const != INDEX_NONE)
		{
			//We've used the same constant before so just use that one.
			varinfo = (variable_info*)hash_table_find(var_info_table, seen_constants[same_const]);
		}
		else
		{
			//TODO: Remove this. We shouldn't need to communicate with the compiler above at all.
			//Internal constants should be written to a header on the bytecode or something and appended to the VM constant table after all the real uniforms.
			//This is just an implementation quirk of the VM. The System above should know nothing about it.

			varinfo = ralloc(mem_ctx, variable_info);
			hash_table_insert(var_info_table, varinfo, constant);
			varinfo->Init(mem_ctx, constant, EVectorVMOperandLocation::Constant);
			int32 offset = CompilationOutput.InternalConstantData.Num();
			varinfo->root->offset = constant_table_size_bytes + offset;
			seen_constants.Add(constant);

			switch (type->base_type)
			{
			case GLSL_TYPE_FLOAT: 
			{
				CompilationOutput.InternalConstantOffsets.Add(offset);
				CompilationOutput.InternalConstantTypes.Add(EVectorVMBaseTypes::Float);
				CompilationOutput.InternalConstantData.AddUninitialized(sizeof(float));
				*(float*)(CompilationOutput.InternalConstantData.GetData() + offset) = constant->value.f[0];
				break;
			}
			case GLSL_TYPE_INT:
			{
				CompilationOutput.InternalConstantOffsets.Add(offset);
				CompilationOutput.InternalConstantTypes.Add(EVectorVMBaseTypes::Int);
				CompilationOutput.InternalConstantData.AddUninitialized(sizeof(int32));
				*(int32*)(CompilationOutput.InternalConstantData.GetData() + offset) = constant->value.i[0];
				break;
			}
			case GLSL_TYPE_BOOL:
			{
				CompilationOutput.InternalConstantOffsets.Add(offset);
				CompilationOutput.InternalConstantTypes.Add(EVectorVMBaseTypes::Bool);
				CompilationOutput.InternalConstantData.AddUninitialized(sizeof(int32));
				*(int32*)(CompilationOutput.InternalConstantData.GetData() + offset) = constant->value.b[0] ? 0xFFFFFFFF : 0x0;
				break;
			}
			default: check(0);
			};
		}

		check(varinfo->root->is_scalar());
		check(varinfo);
		curr_node = varinfo->root;
		curr_node->last_read = ordered_ops.Num() - 1;

		return visit_continue;
	}

	virtual ir_visitor_status visit_enter(ir_dereference_array* array_deref)
	{
		//Only supporting array derefs for matrices atm.
		check(array_deref->array->type->is_matrix());

		//Only support constant matrix indices. Not immediately clear how I'd do non-const access.
		ir_constant* index = array_deref->array_index->as_constant();
		check(index->type->is_integer() && index->type->is_scalar());

		variable_info *var_info = (variable_info*)hash_table_find(var_info_table, array_deref->variable_referenced());
		check(var_info);
		check(curr_node == nullptr);
		curr_node = var_info->root->children[index->get_int_component(0)];
		if (!is_output_param)
		{
			curr_node->last_read = ordered_ops.Num() - 1;
		}

		return visit_continue_with_parent;
	}

	/** ir_dereference_variable marks the start of a dereference chain. So we begin traversing the var_info tree for this variable and store the result in curr_node. */
	virtual ir_visitor_status visit(ir_dereference_variable *deref)
	{
		variable_info *var_info = (variable_info*)hash_table_find(var_info_table, deref->variable_referenced());
		check(var_info);
		check(curr_node == nullptr);

		if (!in_assignee && deref->var->mode == ir_var_out)
		{
			_mesa_glsl_error(parse_state, "Reading from an output variable is invalid.");
		}

		curr_node = var_info->root;
		if (!is_output_param)
		{
			curr_node->last_read = ordered_ops.Num() - 1;
		}
		return visit_continue;
	}

	/** As we leave a swizzle we may need to move down from curr_node into a child node. eg. from SomeVector to SomeVector.x */
	virtual ir_visitor_status visit_leave(ir_swizzle *swiz)
	{
		//All non scalar vars have a swizzle to grab the right component.
		check(swiz->mask.num_components == 1);
		check(!in_assignee);

		//if we hit a swizzle then we must be on the way out of a variable deref which should have set this.
		check(curr_node);

		const glsl_type* val_type = swiz->val->type;
		if (val_type->is_matrix())
		{
			unsigned swiz_comp = swiz->mask.x;
			unsigned matrix_row = swiz_comp / val_type->vector_elements;
			unsigned matrix_col = swiz_comp % val_type->matrix_columns;

			check(matrix_col < curr_node->num_children);
			curr_node = curr_node->children[matrix_row];
			check(matrix_row < curr_node->num_children);
			curr_node = curr_node->children[matrix_col];
		}
		else if (swiz->val->type->is_vector())
		{
			unsigned swiz_comp = swiz->mask.x;
			check(swiz_comp < curr_node->num_children);
			curr_node = curr_node->children[swiz_comp];
		}

		if (!is_output_param)
		{
			curr_node->last_read = ordered_ops.Num() - 1;
		}

		//swizzles must be the final entry in a deref chain so we have to have reached a scalar by now.
		check(curr_node->is_scalar());

		return visit_continue;
	}

	/** As we leave a ir_dereference_record we must move from curr_node into the member dereferenced. E.g. SomeStruct to SomeStruct.Member*/
	virtual ir_visitor_status visit_leave(ir_dereference_record *deref)
	{
		//Must be on the way out of a deref chain here so curr_node should have been set by previous entries in the chain.
		check(curr_node);

		//Find the correct child node to move into.
		variable_info_node* prev_node = curr_node;
		for (unsigned i = 0; i < curr_node->num_children; ++i)
		{
			if (strcmp(curr_node->children[i]->name, deref->field) == 0)
			{
				curr_node = curr_node->children[i];
				break;
			}
		}
		check(prev_node != curr_node);//We have to find a child to move into.

		if (!is_output_param)
		{
			curr_node->last_read = ordered_ops.Num() - 1;
		}

		return visit_continue;
	}

	/** helper function to grab the current op in a neater way. */
	FORCEINLINE op_base* curr_op()
	{
		return ordered_ops.Last();
	}
	
	/** Visiting expressions, here we build ordered_ops. */
	virtual ir_visitor_status visit_enter(ir_expression* expression)
	{
		check(curr_node == nullptr);
		check(expression->type->is_scalar());
		check(!is_output_param)

		EVectorVMBaseTypes BaseType = EVectorVMBaseTypes::Num;
		if (expression->type->is_float())
		{
			BaseType = EVectorVMBaseTypes::Float;
		}
		else if (expression->type->is_integer())
		{
			BaseType = EVectorVMBaseTypes::Int;
		}
		else if (expression->type->is_boolean())
		{
			BaseType = EVectorVMBaseTypes::Bool;
		}

		if (BaseType == EVectorVMBaseTypes::Num)
		{
			_mesa_glsl_error(parse_state, "VectorVM cannot handle type %s", expression->type->name);
			return visit_stop;
		}

		TArray<FVMExpresssionInfo>* VMOps = VMExpressionMap.Find(expression->operation);
		if (VMOps)
		{
			FVMExpresssionInfo* ExprInfo = VMOps->FindByPredicate([&](FVMExpresssionInfo& Info) { return Info == expression; });
			if (ExprInfo)
			{
				op_standard* op = curr_op()->as_standard();
				check(op);

				if (op->op_code != EVectorVMOp::done && op->op_code != EVectorVMOp::mad)
				{
					//we're already inside another expression.
					//Currently we only allow this for mad.
					check(op->op_code == EVectorVMOp::add);
					check(ExprInfo->Op == EVectorVMOp::mul);
					op->op_code = EVectorVMOp::mad;
				}
				else
				{
					op->op_code = ExprInfo->Op;
				}

				for (unsigned operand = 0; operand < expression->get_num_operands(); ++operand)
				{
					expression->operands[operand]->accept(this);
					if (curr_node)
					{
						check(curr_node->is_scalar());
						check(op->num_operands < 3);
						op->source_components[op->num_operands++] = curr_node;
						curr_node = nullptr;
					}
				}

				return visit_continue_with_parent;
			}
		}

		FString ExprString = FString::Printf(TEXT("%s %s"), ANSI_TO_TCHAR(expression->type->name), ANSI_TO_TCHAR(ir_expression::operator_string(expression->operation)));
		ExprString += TEXT("(");
		for (unsigned i = 0; i < expression->get_num_operands(); ++i)
		{
			if (i > 0)
			{
				ExprString += TEXT(", ");
			}

			ExprString += expression->operands[i]->type->name;
		}
		ExprString += TEXT(")");
		_mesa_glsl_error(parse_state, "VM does not have a valid operation for %s", TCHAR_TO_ANSI(*ExprString));
		return visit_stop;
	}

	template<typename T>
	void allocate()
	{
		ordered_ops.Add(new T());
	}
	
	virtual ir_visitor_status visit_enter(ir_call* call)
	{
		check(curr_node == nullptr);

		EVectorVMOp VVMOpCode = get_special_vm_opcode(call->callee);
		if (VVMOpCode == EVectorVMOp::done)
		{
			_mesa_glsl_error(parse_state, "could not find vm opcode for function %s", call->callee_name());
			return visit_stop;
		}

		unsigned num_operands = 0;
		foreach_iter(exec_list_iterator, iter, call->actual_parameters)
		{
			++num_operands;
		}

		switch (VVMOpCode)
		{
		case EVectorVMOp::inputdata_half: allocate<op_input_half>(); break;
		case EVectorVMOp::inputdata_int32: allocate<op_input_int>(); break;
		case EVectorVMOp::inputdata_float: allocate<op_input_float>(); break;
		case EVectorVMOp::inputdata_noadvance_float: allocate<op_input_noadvance_float>(); break;
		case EVectorVMOp::inputdata_noadvance_int32: allocate<op_input_noadvance_int>(); break;
		case EVectorVMOp::inputdata_noadvance_half: allocate<op_input_noadvance_half>(); break;
		case EVectorVMOp::outputdata_half: allocate<op_output_half>(); break;
		case EVectorVMOp::outputdata_int32: allocate<op_output_int>(); break;
		case EVectorVMOp::outputdata_float: allocate<op_output_float>(); break;
		case EVectorVMOp::acquireindex: allocate<op_index_acquire>(); break;
		case EVectorVMOp::acquire_id: allocate<op_id_acquire>(); break;
		case EVectorVMOp::update_id: allocate<op_id_update>(); break;
		case EVectorVMOp::external_func_call: allocate<op_external_func>(); break;

		case EVectorVMOp::enter_stat_scope:
		case EVectorVMOp::exit_stat_scope: allocate<op_hook>(); break;

		default: allocate<op_standard>(); break;
			//todo: other special ops
		};

		curr_op()->op_code = VVMOpCode;
		if (op_standard* standard = curr_op()->as_standard())
		{
			//Handle each param.
			exec_node* curr = call->actual_parameters.get_head();
			exec_node* curr_param = call->callee->parameters.get_head();
			for (unsigned param_idx = 0; param_idx < num_operands; ++param_idx)
			{
				check(curr);
				ir_rvalue* param = (ir_rvalue*)curr;
				ir_variable* real_param = (ir_variable*)curr_param;

				is_output_param = real_param->mode == ir_var_out;
				param->accept(this);
				is_output_param = false;

				check(curr_node);
				check(curr_node->is_scalar());
				check(standard->num_operands < 3);
				standard->source_components[standard->num_operands++] = curr_node;
				curr_node = nullptr;

				curr = curr->get_next();
				curr_param = curr_param->get_next();
			}
			standard->num_operands = num_operands;

			//Adjust noise opcode.
			if (VVMOpCode == EVectorVMOp::noise)
			{
				switch (standard->num_operands)
				{
				case 1: standard->op_code = EVectorVMOp::noise; break;
				case 2: standard->op_code = EVectorVMOp::noise2D; break;
				case 3: standard->op_code = EVectorVMOp::noise3D; break;
				default:
					_mesa_glsl_error(parse_state, "Invalid number of params to noise. Only 1, 2 and 3D noise is available.");
					break;
				}
			}

			//Handle the dest
			check(call->return_deref);
			in_assignee = true;
			call->return_deref->accept(this);
			in_assignee = false;

			check(curr_node && curr_node->is_scalar());
			standard->dest_component = curr_node;
			curr_node = nullptr;
		}
		else if (op_hook* hook = curr_op()->as_hook())
		{
			check(num_operands <= 1);

			if (num_operands > 0)
			{
				exec_node* curr = call->actual_parameters.get_head();
				check(curr);
				ir_rvalue* param = (ir_rvalue*)curr;
				param->accept(this);

				check(curr_node);
				check(curr_node->is_scalar());
				hook->source_component = curr_node;
				curr_node = nullptr;
			}			
		}
		else if (op_input* input = curr_op()->as_input())
		{
			check(num_operands == 2);
			//Handle the dataset index.
			exec_node* curr = call->actual_parameters.get_head();
			check(curr);
			ir_constant* dataset_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(dataset_idx_const);
			input->dataset_idx = dataset_idx_const->get_int_component(0);

			//Handle the register index.
			curr = curr->get_next();
			check(curr);
			ir_constant* reg_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(reg_idx_const);
			input->register_idx = reg_idx_const->get_int_component(0);

			//TODO: handle instance_idx_component for indexed input reads.

			//Handle the dest
			check(call->return_deref);
			in_assignee = true;
			call->return_deref->accept(this);
			in_assignee = false;
			check(curr_node && curr_node->is_scalar());
			input->dest_component = curr_node;
			curr_node = nullptr;
		}
		else if (op_output* output = curr_op()->as_output())
		{
			check(num_operands == 4);
			//Handle the data set index.
			exec_node* curr = call->actual_parameters.get_head();
			check(curr);
			ir_constant* set_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(set_idx_const);
			output->dataset_index = set_idx_const->get_int_component(0);

			//Handle the register index.
			curr = curr->get_next();
			check(curr);
			ir_constant* reg_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(reg_idx_const);
			output->register_idx = reg_idx_const->get_int_component(0);

			//Handle the instance index.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* instance_idx = (ir_rvalue*)curr;
			instance_idx->accept(this);
			check(curr_node);
			output->instance_idx_component = curr_node;
			curr_node = nullptr;

			//Handle data we're outputting.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* value = (ir_rvalue*)curr;
			value->accept(this);
			check(curr_node);
			output->value_component = curr_node;
			curr_node = nullptr;

			check(!call->return_deref);
		}
		else if (op_index_acquire* acquire = curr_op()->as_index_acquire())
		{
			check(num_operands == 2);

			//Handle dataset index
			exec_node* curr = call->actual_parameters.get_head();
			check(curr);
			ir_constant* dataset_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(dataset_idx_const);
			acquire->dataset_idx = dataset_idx_const->get_int_component(0);

			//Handle valid parameter. Can be constant or variable.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* valid = (ir_rvalue*)curr;
			valid->accept(this);
			check(curr_node);
			acquire->valid_component = curr_node;
			curr_node = nullptr;

			//Handle dest.
			check(call->return_deref);
			in_assignee = true;
			call->return_deref->accept(this);
			in_assignee = false;
			check(curr_node && curr_node->is_scalar());
			acquire->dest_component = curr_node;
			curr_node = nullptr;
		}
		else if (op_id_acquire* id_acquire = curr_op()->as_id_acquire())
		{
			check(num_operands == 3);

			//Handle dataset index
			exec_node* curr = call->actual_parameters.get_head();
			check(curr);
			ir_constant* dataset_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(dataset_idx_const);
			id_acquire->dataset_idx = dataset_idx_const->get_int_component(0);

			//Handle id index.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* id_index = (ir_rvalue*)curr;
			id_index->accept(this);
			check(curr_node);
			id_acquire->id_index_component = curr_node;
			curr_node = nullptr;

			//Handle id tag.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* id_tag = (ir_rvalue*)curr;
			id_tag->accept(this);
			check(curr_node);
			id_acquire->id_tag_component = curr_node;
			curr_node = nullptr;
		}
		else if (op_id_update* update = curr_op()->as_id_update())
		{
			check(num_operands == 3);

			//Handle dataset index
			exec_node* curr = call->actual_parameters.get_head();
			check(curr);
			ir_constant* dataset_idx_const = ((ir_rvalue*)curr)->as_constant();
			check(dataset_idx_const);
			update->dataset_idx = dataset_idx_const->get_int_component(0);

			//Handle ID parameter.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* id = (ir_rvalue*)curr;
			id->accept(this);
			check(curr_node);
			update->id_component = curr_node;
			curr_node = nullptr;

			//Handle index parameter.
			curr = curr->get_next();
			check(curr);
			ir_rvalue* index = (ir_rvalue*)curr;
			index->accept(this);
			check(curr_node);
			update->index_component = curr_node;
			curr_node = nullptr;
		}
		else if (op_external_func* func = curr_op()->as_external_func())
		{
			unsigned num_inputs = 0;
			unsigned num_outputs = 0;

			func->sig = call->callee;

			exec_node* sig_param_node = call->callee->parameters.get_head();
			foreach_iter(exec_list_iterator, iter, call->actual_parameters)
			{
				ir_rvalue* param = (ir_rvalue*)(iter.get());
				check(sig_param_node);
				ir_variable* sig_param = (ir_variable*)sig_param_node;

				check(curr_node == nullptr);
				is_output_param = sig_param->mode == ir_var_out;
				param->accept(this);
				check(curr_node != nullptr);	
				is_output_param = false;

				if (sig_param->mode == ir_var_in)
				{
					func->inputs.Add(curr_node);
				}
				else
				{
					func->outputs.Add(curr_node);
				}	
				curr_node = nullptr;
				sig_param_node = sig_param_node->get_next();
			}
		}
		else
		{
			check(0);
		}

		return visit_continue_with_parent;
	}

	/** Visiting assignments generates a new op entry and visits it's lhs to populate the dest_component and the RHS to populate the source_components. */
	virtual ir_visitor_status visit_enter(ir_assignment *assign)
	{
		ir_variable* dest_var = assign->lhs->variable_referenced();
		check(dest_var);

		//All assignments are standard ops currently.
		allocate<op_standard>();
		op_base* base = curr_op();
		op_standard* op = base->as_standard();

		dest_component = get_dest_comp(assign->write_mask);

		//Visit the LHS to grab the right var_info_node and generate the new op's dest data.
		curr_node = nullptr;
		in_assignee = true;
		assign->lhs->accept(this);
		check(curr_node);

		if (!curr_node->is_scalar())
		{
			//If curr node isn't scalar we may be able to drill into a scalar child based on the write mask.		
			check(dest_component < curr_node->num_children);
			curr_node = curr_node->children[dest_component];
		}

		check(curr_node && curr_node->is_scalar());
		op->dest_component = curr_node;

		in_assignee = false;

		//Visit the RHS to generate the op info for each source component / operand.
		curr_node = nullptr;
		assign->rhs->accept(this);

// 		//If the RHS was just a dereference chain rather than an expression, curr_node will be valid and this op must be an output so generate that now.
// 		if (curr_node)
// 		{
// 			check(op->num_operands == 0);
// 			check(curr_node->is_scalar());
// 			check(dest_var->mode == ir_var_out);//we must be writing to an output variable.
// 			op->op_code = EVectorVMOp::output;
// 			op->source_components[op->num_operands++] = curr_node;
// 			curr_node = nullptr;
// 		}

		return visit_continue_with_parent;
	}

	void Finalize()
	{
		//Allocate temp registers
// 		const unsigned num_temp_registers = VectorVM::NumTempRegisters;
// 		unsigned *registers = rzalloc_array(mem_ctx, unsigned, num_temp_registers);
// 		FMemory::Memset(registers, 0xFF, sizeof(unsigned) * num_temp_registers);
		TArray<uint32> temp_registers;
		temp_registers.Reserve(100);
#if VM_VERBOSE_LOGGING
		UE_LOGF(LogVVMBackend, Log, "\n-------------------------------\nTemporary Allocations\n------------------------------\n");
#endif
		TArray<op_base*> stripped_ops;
		stripped_ops.Reserve(ordered_ops.Num());
		for (int32 i = 0; i < ordered_ops.Num(); ++i)
		{
			if (ordered_ops[i]->finalize(parse_state, CompilationOutput, temp_registers, i))
			{
				stripped_ops.Add(ordered_ops[i]);
				ordered_ops[i]->validate(parse_state, i);
			}
		}

		if (!parse_state->error)
		{
			//Now emit the bytecode
			for (int32 op_idx = 0; op_idx < stripped_ops.Num(); ++op_idx)
			{
				stripped_ops[op_idx]->add_to_bytecode(CompilationOutput.ByteCode);
			}

			emit_byte((uint8)EVectorVMOp::done, CompilationOutput.ByteCode);
		}

		ordered_ops = stripped_ops;
	}

	FString DumpOps(uint32& OutNumOps)
	{
		FString OpStr;
		OutNumOps = ordered_ops.Num();

		//First go through all the parameters.
		FString OpsConstantTable;
		uint32 NumConstants = 0;
		TFunction<void(variable_info_node*, FString)> GatherConstantTable = [&](variable_info_node* node, FString accumulated_name)
		{
			accumulated_name += node->name;
			if (node->is_scalar())
			{
				check(node->offset != INDEX_NONE);
				OpsConstantTable += FString::Printf(TEXT("%d | %s\n"), node->offset, *accumulated_name);
				NumConstants++;
			}
			else
			{
				for (unsigned i = 0; i < node->num_children; ++i)
				{
					GatherConstantTable(node->children[i], accumulated_name);
				}
			}
		};

		for(variable_info* varinfo : param_vars)
		{
			GatherConstantTable(varinfo->root, FString());
		}

		//Then go through all the internal constants.
		for (int32 i = 0; i < CompilationOutput.InternalConstantOffsets.Num(); ++i)
		{
			EVectorVMBaseTypes Type = CompilationOutput.InternalConstantTypes[i];
			int32 Offset = CompilationOutput.InternalConstantOffsets[i];
			int32 TableOffset = constant_table_size_bytes + Offset;
			switch (Type)
			{
			case EVectorVMBaseTypes::Float:
			{
				float Val = *(float*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				OpsConstantTable += FString::Printf(TEXT("%d | %.9g\n"), TableOffset, Val);
				NumConstants++;
			}
			break;
			case EVectorVMBaseTypes::Int:
			{
				int32 Val = *(int32*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				OpsConstantTable += FString::Printf(TEXT("%d | %d\n"), TableOffset, Val);
				NumConstants++;
			}
			break;
			case EVectorVMBaseTypes::Bool:
			{
				int32 Val = *(int32*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				OpsConstantTable += FString::Printf(TEXT("%d | %s\n"), TableOffset, Val == INDEX_NONE ? TEXT("True") : (Val == 0 ? TEXT("False") : TEXT("Invalid") ));
				NumConstants++;
			}
			break;
			}
		}
		
		
		OpStr += TEXT("\n-------------------------------\n");
		OpStr += TEXT("Summary\n");
		OpStr += TEXT("-------------------------------\n");
		OpStr += FString::Printf(TEXT("Num Byte Code Ops: %d\n"), ordered_ops.Num());
		OpStr += FString::Printf(TEXT("Num Constants: %d\n"), NumConstants);
		
		//Dump the constant table
		OpStr += TEXT("\n-------------------------------\n");
		OpStr += TEXT("Constant Table\n");
		OpStr += TEXT("-------------------------------\n");
		OpStr += OpsConstantTable;

		OpStr += TEXT("-------------------------------\n");
		OpStr += FString::Printf(TEXT("Byte Code (%d Ops)\n"), ordered_ops.Num());
		OpStr += TEXT("-------------------------------\n");
		
		//Dump the bytecode		
		for (int32 op_idx = 0; op_idx < ordered_ops.Num(); ++op_idx)
		{
			OpStr += FString::Printf(TEXT("%d\t| "),op_idx) + ordered_ops[op_idx]->to_string();
		}
		
		OpStr += TEXT("-------------------------------\n");

#if VM_VERBOSE_LOGGING
		UE_LOGF(LogVVMBackend, Log, "\n%ls", *OpStr);
#endif
		return OpStr;
	}

public:
	static char* run(exec_list *ir, _mesa_glsl_parse_state *state, FVectorVMCompilationOutput& InCompOutput)
	{
		//now visit all assignments and gather operations info.
		ir_gen_vvm_visitor genv(state, InCompOutput);

		//Have to manually visit the uniform variables from cbuffers first.		
		for (SCBuffer& buff : state->CBuffersStructuresFlattened)
		{
			for (SCBufferMember& member : buff.Members)
			{
				member.Var->accept(&genv);
			}
		}
		visit_list_elements(&genv, ir);
		genv.Finalize();

		InCompOutput.AssemblyAsString = genv.DumpOps(InCompOutput.NumOps);

		return NULL;
	}
};


void vm_gen_bytecode(exec_list *ir, _mesa_glsl_parse_state *state, FVectorVMCompilationOutput& InCompOutput)
{
	BuildExpressionMap();

	ir_gen_vvm_visitor::run(ir, state, InCompOutput);
}