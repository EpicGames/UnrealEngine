// Copyright Epic Games, Inc. All Rights Reserved.

#include "ir_vm_helper.h"

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

namespace ir_vm_helper
{

unsigned get_component_from_matrix_array_deref(ir_dereference_array* array_deref)
{
	check(array_deref);
	check(array_deref->variable_referenced()->type->is_matrix());
	ir_constant* index = array_deref->array_index->as_constant();
	check((index->type == glsl_type::uint_type || index->type == glsl_type::int_type) && index->type->is_scalar());
	unsigned deref_idx = index->type == glsl_type::uint_type ? index->value.u[0] : index->value.i[0];
	return deref_idx * array_deref->variable_referenced()->type->vector_elements;
}

ir_variable* get_rvalue_variable(ir_rvalue* rvalue, unsigned int& search_comp)
{
	ir_dereference* deref = rvalue->as_dereference();
	ir_dereference_array* array_deref = rvalue->as_dereference_array();
	ir_swizzle* swiz = rvalue->as_swizzle();

	ir_variable* search_var = rvalue->variable_referenced();
	search_comp = 0;

	if (swiz)
	{
		if (ir_dereference_array* swiz_array_deref = swiz->val->as_dereference_array())
		{
			search_comp = get_component_from_matrix_array_deref(swiz_array_deref);
		}
		search_comp += swiz->mask.x;
	}
	else if (array_deref)
	{
		//We can only handle matrix array derefs but these will have an outer swizzle that we'll work with. 
		check(array_deref->array->type->is_matrix());
		search_var = nullptr;
	}
	else if (!deref || !deref->type->is_scalar())
	{
		//If we're not a deref or we're not a straight scalar deref then we should leave this alone.
		search_var = nullptr;
	}

	return search_var;
}

};

