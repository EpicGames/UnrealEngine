// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class ir_dereference_array;
class ir_rvalue;
class ir_variable;

namespace ir_vm_helper
{

unsigned get_component_from_matrix_array_deref(ir_dereference_array* array_deref);
ir_variable* get_rvalue_variable(ir_rvalue* rvalue, unsigned int& search_comp);

};

