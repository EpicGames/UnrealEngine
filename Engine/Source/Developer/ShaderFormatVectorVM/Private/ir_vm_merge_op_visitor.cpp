// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

#include "VectorVM.h"



/** finds any group of ops than can be merged into a single compound operation. Mad for example. */
class ir_merge_op_visitor final : ir_hierarchical_visitor
{
	_mesa_glsl_parse_state* state;
	bool assign_has_expressions;

	bool progress;
	unsigned expr_depth;
	TArray<ir_assignment*> mul_assignments;
	TArray<ir_assignment*> neg_assignments;

	TArray<ir_assignment*>* assign_array_to_add;

public:

	ir_merge_op_visitor(_mesa_glsl_parse_state* in_state)
	{
		state = in_state;
		assign_has_expressions = false;
		progress = false;
		assign_array_to_add = NULL;
		expr_depth = 0;
	}

	virtual ~ir_merge_op_visitor()
	{
	}

	virtual ir_visitor_status visit_enter(ir_expression* expr) override
	{
		++expr_depth;
		check(expr_depth == 1);//Should never exceed one expression.
		return visit_continue;
	}
	virtual ir_visitor_status visit_leave(ir_expression* expr) override
	{
		if (expr->operation == ir_binop_mul)
		{
			assign_array_to_add = &mul_assignments;
		}
		else if (expr->operation == ir_unop_neg)
		{
			assign_array_to_add = &neg_assignments;
		}
		else if (expr_depth == 1)
		{
			check(assign_array_to_add == nullptr);

			if (expr->operation == ir_binop_add)
			{
				int32 equiv_operand = 0;
				int32 mul_idx = mul_assignments.IndexOfByPredicate(
					[&](ir_assignment* test_assign)
				{
					if (AreEquivalent(test_assign->lhs, expr->operands[0]))
					{
						equiv_operand = 0;
						return true;
					}
					else if (AreEquivalent(test_assign->lhs, expr->operands[1]))
					{
						equiv_operand = 1;
						return true;
					}
					return false;
				}
				);

				if (mul_idx != INDEX_NONE)
				{
					ir_assignment* assign = mul_assignments[mul_idx];
					if (assign->lhs->type->is_float())
					{
						ir_rvalue* mul_operand = assign->rhs->clone(state, NULL);
						ir_rvalue* add_operand = equiv_operand == 0 ? expr->operands[1] : expr->operands[0];

						expr->operands[0] = mul_operand;
						expr->operands[1] = add_operand;

						mul_assignments.RemoveAtSwap(mul_idx);
						progress = true;
					}
				}
				else
				{
					int32 neg_idx = neg_assignments.IndexOfByPredicate(
						[&](ir_assignment* test_assign)
					{
						if (AreEquivalent(test_assign->lhs, expr->operands[0]))
						{
							equiv_operand = 0;
							return true;
						}
						else if (AreEquivalent(test_assign->lhs, expr->operands[1]))
						{
							equiv_operand = 1;
							return true;
						}
						return false;
					}
					);

					if (neg_idx != INDEX_NONE)
					{
						ir_assignment* assign = neg_assignments[neg_idx];

						ir_rvalue *inner_val = nullptr;
						ir_expression* neg_expr = nullptr;
						if (ir_swizzle* swiz = assign->rhs->as_swizzle())
						{
							neg_expr = swiz->val->as_expression();
							check(neg_expr);
						}
						else
						{
							neg_expr = assign->rhs->as_expression();
							check(neg_expr);
						}
						inner_val = neg_expr->operands[0];
						check(inner_val);

						ir_rvalue* neg_operand = inner_val->clone(state, NULL);
						ir_rvalue* add_operand = equiv_operand == 0 ? expr->operands[1] : expr->operands[0];

						expr->operation = ir_binop_sub;
						expr->operands[0] = add_operand;
						expr->operands[1] = neg_operand;

						neg_assignments.RemoveAtSwap(neg_idx);
						progress = true;
					}
				}
			}
		}

		--expr_depth;

		return visit_continue;
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign) override
	{
		assign_array_to_add = nullptr;
		expr_depth = 0;
		assign->rhs->accept(this);

		if (assign_array_to_add)
		{
			assign_array_to_add->Add(assign);
		}

		return visit_continue_with_parent;
	}

	static void run(exec_list *ir, _mesa_glsl_parse_state *state)
	{
		ir_merge_op_visitor v(state);
		visit_list_elements(&v, ir);
	}
};


void vm_merge_ops(exec_list *ir, _mesa_glsl_parse_state *state)
{
	ir_merge_op_visitor::run(ir, state);
}