// Copyright Epic Games, Inc. All Rights Reserved.


#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

#include "ir_vm_helper.h"
#include "VectorVM.h"

// before we worry about propagating expressions we want to clean up our existing assignments
// so that we get rid of any redundant assignments (A = A) as well as move towards an SSA
// form so we avoid 'A = B; A = C' (instead prefering A = B; D = C).  This will allow our existing
// propagation visitors to work.
class ir_ssa_visitor final : public ir_rvalue_visitor
{
public:
		_mesa_glsl_parse_state* parse_state;
	bool progress = false;

	ir_ssa_visitor(_mesa_glsl_parse_state* in_state)
	{
		parse_state = in_state;
	}

	using ScalarVariable = TTuple<ir_variable*, int32>;
	TMap<ScalarVariable, ir_rvalue*> ReplacementMap;

	virtual void handle_rvalue(ir_rvalue** rvalue) override
	{
		if (rvalue && *rvalue)
		{
			uint32 ComponentIndex = 0;
			if (ir_variable* Variable = ir_vm_helper::get_rvalue_variable(*rvalue, ComponentIndex))
			{
				if (ir_rvalue** Replacement = ReplacementMap.Find(MakeTuple(Variable, ComponentIndex)))
				{
					if (*Replacement)
					{
						ir_rvalue* NewRValue = (*Replacement)->clone(parse_state, nullptr);
						*rvalue = NewRValue;
						progress = true;
					}
				}
			}
		}
	}

	virtual ir_visitor_status visit_leave(ir_assignment* assign) override
	{
		ir_variable* lhs = assign->lhs->variable_referenced();

		const int32 ComponentIndex = FMath::CountTrailingZeros(assign->write_mask);
		check((1 << ComponentIndex) == assign->write_mask);

		// remove redundant assignments
		uint32 rhs_component_index = 0;
		if (const ir_variable* rhsVariable = ir_vm_helper::get_rvalue_variable(assign->rhs, rhs_component_index))
		{
			if (lhs == rhsVariable && rhs_component_index == ComponentIndex)
			{
				assign->remove();
				progress = true;
				return visit_continue;
			}
		}

		if (ir_rvalue** Existing = ReplacementMap.Find(MakeTuple(lhs, ComponentIndex)))
		{
			// we're already writing to this variable, so we're going to create a new variable for
			// the LHS, and replace all subsequent matches to our new variable
			ir_variable* NewVariable = new(parse_state) ir_variable(lhs->type->get_base_type(), "SSA_tmp", ir_var_temporary);
			ir_rvalue* NewLhs = new(parse_state) ir_dereference_variable(NewVariable);

			base_ir->insert_before(NewVariable);
			assign->set_lhs(NewLhs);
			assign->write_mask = 1;

			*Existing = NewLhs;
			progress = true;
		}
		else
		{
			ReplacementMap.Add(MakeTuple(lhs, ComponentIndex), nullptr);
		}

		return ir_rvalue_visitor::visit_leave(assign);
	}

	static void run(exec_list* ir, _mesa_glsl_parse_state* state)
	{
		bool progress = false;
		do
		{
			ir_ssa_visitor ssa_visitor(state);
			visit_list_elements(&ssa_visitor, ir);

			progress = ssa_visitor.progress;

			progress = do_dead_code(ir, false) || progress;
			progress = do_dead_code_local(ir) || progress;
		} while (progress);
	}
};

/** Removes any assignments that don't actually map to a VM op but just move some data around. We look for refs and grab the source data direct. */
class ir_propagate_non_expressions_visitor final : public ir_rvalue_visitor
{
	_mesa_glsl_parse_state* parse_state;

	struct var_info
	{
		int32 latest_expr_assign[16];
		int32 latest_non_expr_assign[16];
		var_info()
		{
			FMemory::Memset(latest_expr_assign, 0xFF, sizeof(latest_expr_assign));
			FMemory::Memset(latest_non_expr_assign, 0xFF, sizeof(latest_non_expr_assign));
		}
	};
	TMap<ir_variable*, var_info> var_info_map;

	TArray<ir_assignment*> assignments;
	TArray<bool> PreserveAssignment;

	int num_expr;
	bool progress;
	TArray<ir_assignment*> non_expr_assignments;

public:
	ir_propagate_non_expressions_visitor(_mesa_glsl_parse_state* in_state)
	{
		parse_state = in_state;
		num_expr = 0;
		progress = false;
	}

	virtual ~ir_propagate_non_expressions_visitor()
	{
	}

	unsigned get_component_from_matrix_array_deref(ir_dereference_array* array_deref) const
	{
		check(array_deref);
		check(array_deref->variable_referenced()->type->is_matrix());
		ir_constant* index = array_deref->array_index->as_constant();
		check((index->type == glsl_type::uint_type || index->type == glsl_type::int_type) && index->type->is_scalar());
		unsigned deref_idx = index->type == glsl_type::uint_type ? index->value.u[0] : index->value.i[0];
		return deref_idx * array_deref->variable_referenced()->type->vector_elements;
	}

	virtual void handle_rvalue(ir_rvalue** rvalue)
	{
		if (rvalue && *rvalue && !in_assignee)
		{
			ir_rvalue** to_replace = rvalue;

			unsigned int search_comp = 0;
			ir_variable* search_var = ir_vm_helper::get_rvalue_variable(*rvalue, search_comp);

			//Search to see if this deref matches any of the non-expression assignments LHS. If so then clone the rhs in it's place.

			var_info* varinfo = var_info_map.Find(search_var);
			if (varinfo)
			{
				//Is there a previous non_expr assignment after any containing expressions?
				//If so, copy that in place of this rvalue.
				if (varinfo->latest_expr_assign[search_comp] < varinfo->latest_non_expr_assign[search_comp])
				{
					ir_assignment* assign = assignments[varinfo->latest_non_expr_assign[search_comp]];
					check(assign->rhs->as_expression() == nullptr);
					check(assign->rhs->as_swizzle() || assign->rhs->as_dereference_variable() || assign->rhs->as_constant() || assign->rhs->as_dereference_record());
					check(assign->rhs->type->is_scalar());//All assignments must be scalar at this point!
					ir_rvalue* new_rval = assign->rhs->clone(parse_state, nullptr);
					(*rvalue) = new_rval;
					progress = true;
				}
				else if (varinfo->latest_expr_assign[search_comp] != INDEX_NONE)
				{
					PreserveAssignment[varinfo->latest_expr_assign[search_comp]] = true;
				}
			}
		}
	}

	virtual ir_visitor_status visit_enter(ir_expression*) override
	{
		num_expr++;
		return visit_continue;
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign) override
	{
		if (assign->condition)
		{
			_mesa_glsl_error(parse_state, "conditional assignment in instruction stream");
			return visit_stop;
		}

		num_expr = 0;
		return visit_continue;
	}

	virtual ir_visitor_status visit_leave(ir_assignment* assign)
	{
		check(assign->next && assign->prev);
		ir_variable* lhs = assign->lhs->variable_referenced();
		var_info& varinfo = var_info_map.FindOrAdd(lhs);

		int32 assign_idx = assignments.Add(assign);
		bool& Preserve = PreserveAssignment.Add_GetRef(false);

		//Add any new temp or auto assignments. These will be grabbed later to use in replacements in HandleRValue.

		unsigned assign_comp = 0;
		if (ir_dereference_array* array_deref = assign->lhs->as_dereference_array())
		{
			assign_comp += get_component_from_matrix_array_deref(array_deref);
		}

		// if our LHS is the RHS of an assignment that is set to copy forward the reference, then we need to invalidate
		// it so that we don't copy forward a different result
		for (int32 previous_assign_idx = assign_idx - 1; previous_assign_idx >= 0; --previous_assign_idx)
		{
			uint32 rhs_component = 0;
			ir_variable* rhs_variable = ir_vm_helper::get_rvalue_variable(assignments[previous_assign_idx]->rhs, rhs_component);
			if (rhs_variable == lhs && ((1 << rhs_component) & assign->write_mask))
			{
				ir_variable* previous_lhs = assignments[previous_assign_idx]->lhs->variable_referenced();
				if (var_info* previous_varinfo = var_info_map.Find(previous_lhs))
				{
					previous_varinfo->latest_expr_assign[rhs_component] = assign_idx;
					PreserveAssignment[previous_assign_idx] = true;
					break;
				}
			}
		}

		unsigned write_mask = assign->write_mask;
		int32 components_written = 0;
		while (write_mask)
		{
			if (write_mask & 0x1)
			{
				++components_written;
				ir_variable_mode mode = lhs->mode;
				if (num_expr == 0)
				{
					unsigned int rhs_component = 0;
					ir_variable* rhs_variable = ir_vm_helper::get_rvalue_variable(assign->rhs, rhs_component);

					// handle the case of redundant self assignment
					if (assign_comp != rhs_component || lhs != rhs_variable)
					{
						varinfo.latest_non_expr_assign[assign_comp] = assign_idx;
					}
				}
				else
				{
					check(mode == ir_var_temporary || mode == ir_var_auto);//We can only perform expressions on temp or auto variables.
					varinfo.latest_expr_assign[assign_comp] = assign_idx;
					Preserve = true;
				}
			}
			++assign_comp;
			write_mask >>= 1;
		}
		check(components_written == 1);
		check(assign->rhs->type->is_scalar());
		return ir_rvalue_visitor::visit_leave(assign);
	}

	void Finalize()
	{
		const int32 AssignmentCount = assignments.Num();
		for (int32 AssignmentIt = 0; AssignmentIt < AssignmentCount; ++AssignmentIt)
		{
			if (!PreserveAssignment[AssignmentIt])
			{
				assignments[AssignmentIt]->remove();
			}
		}
	}

	static void run(exec_list* ir, _mesa_glsl_parse_state* state)
	{
		bool progress = false;
		do
		{
			ir_ssa_visitor ssa_visitor(state);
			visit_list_elements(&ssa_visitor, ir);

			ir_propagate_non_expressions_visitor propagate_non_expressions_visitor(state);
			visit_list_elements(&propagate_non_expressions_visitor, ir);

			propagate_non_expressions_visitor.Finalize();

			progress = propagate_non_expressions_visitor.progress;

			//vm_debug_print("== propagate non expressions - BEFORE DEADCODE ==\n");
			//vm_debug_dump(ir, state);

			progress = do_dead_code(ir, false) || progress;
			progress = do_dead_code_local(ir) || progress;
		} while (progress);
	}
};

void vm_propagate_non_expressions_visitor(exec_list* ir, _mesa_glsl_parse_state* state)
{
	ir_propagate_non_expressions_visitor::run(ir, state);
}

struct MatrixVectors
{
	ir_variable* v[4];
};

class ir_matrices_to_vectors : public ir_rvalue_visitor
{
	bool progress;
	_mesa_glsl_parse_state* parse_state;

	TMap<class ir_variable*, MatrixVectors> MatrixVectorMap;
public:
	ir_matrices_to_vectors(_mesa_glsl_parse_state* in_parse_state)
	{
		progress = false;
		parse_state = in_parse_state;
	}

	virtual ~ir_matrices_to_vectors()
	{
	}

	virtual ir_visitor_status visit_enter(ir_call* call) override
	{
		bool bFoundMatrixParam = false;
		foreach_iter(exec_list_iterator, iter, call->actual_parameters)
		{
			ir_rvalue* param = (ir_rvalue*)iter.get();
			if (param->type->is_matrix())
			{
				bFoundMatrixParam = true;
				break;
			}
		}

		if (!bFoundMatrixParam)
		{
			return ir_rvalue_visitor::visit_enter(call);
		}

		exec_list old_params = call->actual_parameters;

		call->actual_parameters.make_empty();

		//Replace any matrix params with params for their vector components.
		//This will all be full scalarized to float derefs anyway in later passes.
		auto add_param = [&](ir_rvalue* rval)
		{
			if (rval->type->is_matrix())
			{
				if (rval->variable_referenced()->mode == ir_var_uniform)
				{
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_array(rval->variable_referenced(), new(parse_state) ir_constant(0)));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_array(rval->variable_referenced(), new(parse_state) ir_constant(1)));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_array(rval->variable_referenced(), new(parse_state) ir_constant(2)));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_array(rval->variable_referenced(), new(parse_state) ir_constant(3)));
				}
				else
				{
					MatrixVectors& mv = MatrixVectorMap.FindChecked(rval->variable_referenced());
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_variable(mv.v[0]));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_variable(mv.v[1]));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_variable(mv.v[2]));
					call->actual_parameters.push_tail(new(parse_state) ir_dereference_variable(mv.v[3]));
				}
			}
			else
			{
				call->actual_parameters.push_tail(rval);
			}
		};

		foreach_iter(exec_list_iterator, param_iter, old_params)
		{
			ir_rvalue* param = (ir_rvalue*)param_iter.get();
			add_param(param);
		}

		return visit_continue;
	}

	virtual void handle_rvalue(ir_rvalue** rvalue)
	{
		if (rvalue && *rvalue)
		{
			if (ir_dereference* deref = (*rvalue)->as_dereference_array())
			{
				ir_variable* var = deref->variable_referenced();
				if (var->type->is_matrix() && var->mode != ir_var_uniform)
				{
					MatrixVectors& mv = MatrixVectorMap.FindChecked(var);
					if (ir_dereference_array* array_deref = (*rvalue)->as_dereference_array())
					{
						ir_constant* idx = array_deref->array_index->as_constant();
						check(idx);
						unsigned v_idx = idx->value.i[0];
						//void* p = ralloc_parent(array_deref);
						*rvalue = new(parse_state) ir_dereference_variable(mv.v[v_idx]);
					}
					else if (ir_swizzle* swiz = (*rvalue)->as_swizzle())
					{
						unsigned v_idx = swiz->mask.x / 4;
						swiz->mask.x %= 4;
						//void* p = ralloc_parent(swiz);
						swiz->val = new(parse_state) ir_dereference_variable(mv.v[v_idx]);
					}
				}
			}
		}
	}

	virtual ir_visitor_status visit_leave(ir_assignment* assign)
	{
		//Dont think this is required.
		//Previous pass should convert all matrix ops to vec ops so the worst this will be is a Mat[row] = somevec.xyzw so replacing the array deref is is fine.
		ir_variable* var = assign->lhs->variable_referenced();
		if (var->type->is_matrix())
		{
			MatrixVectors& mv = MatrixVectorMap.FindChecked(var);
			if (ir_dereference_array* array_deref = assign->lhs->as_dereference_array())
			{
				ir_constant* idx = array_deref->array_index->as_constant();
				check(idx);
				unsigned v_idx = idx->value.i[0];
				//void* p = ralloc_parent(array_deref);
				assign->set_lhs(new(parse_state) ir_dereference_variable(mv.v[v_idx]));
			}
			else if (ir_dereference_variable* matrix_deref = assign->lhs->as_dereference_variable())
			{
				if (ir_dereference_variable* src_mat = assign->rhs->as_dereference_variable())
				{
					check(src_mat->type->is_matrix());
					ir_dereference* src_derefs[4];
					//void* p = ralloc_parent(array_deref);
					if (src_mat->variable_referenced()->mode == ir_var_uniform)
					{
						src_derefs[0] = new(parse_state) ir_dereference_array(src_mat->variable_referenced(), new(parse_state) ir_constant(0));
						src_derefs[1] = new(parse_state) ir_dereference_array(src_mat->variable_referenced(), new(parse_state) ir_constant(1));
						src_derefs[2] = new(parse_state) ir_dereference_array(src_mat->variable_referenced(), new(parse_state) ir_constant(2));
						src_derefs[3] = new(parse_state) ir_dereference_array(src_mat->variable_referenced(), new(parse_state) ir_constant(3));

					}
					else
					{
						MatrixVectors& src_mv = MatrixVectorMap.FindChecked(src_mat->variable_referenced());
						src_derefs[0] = new(parse_state) ir_dereference_variable(src_mv.v[0]);
						src_derefs[1] = new(parse_state) ir_dereference_variable(src_mv.v[1]);
						src_derefs[2] = new(parse_state) ir_dereference_variable(src_mv.v[2]);
						src_derefs[3] = new(parse_state) ir_dereference_variable(src_mv.v[3]);
					}

					assign->insert_before(new(parse_state) ir_assignment(new(parse_state) ir_dereference_variable(mv.v[0]), src_derefs[0], assign->condition, 0xF));
					assign->insert_before(new(parse_state) ir_assignment(new(parse_state) ir_dereference_variable(mv.v[1]), src_derefs[1], assign->condition, 0xF));
					assign->insert_before(new(parse_state) ir_assignment(new(parse_state) ir_dereference_variable(mv.v[2]), src_derefs[2], assign->condition, 0xF));
					assign->insert_before(new(parse_state) ir_assignment(new(parse_state) ir_dereference_variable(mv.v[3]), src_derefs[3], assign->condition, 0xF));
				}
				else
				{
					//Pull the vectors out of the matrix into the new vector temporaries.
					ir_constant* mat_const = assign->rhs->as_constant();
					check(mat_const);

					const glsl_type* mat_type = mat_const->type;					
					for(unsigned mat_col = 0; mat_col < mat_type->matrix_columns; ++mat_col)
					{
						ir_constant_data new_vec_data;
						memset(&new_vec_data, 0, sizeof(new_vec_data));
						for (unsigned vec_elem = 0; vec_elem < mat_type->vector_elements; ++vec_elem)
						{
							new_vec_data.f[vec_elem] = mat_const->value.f[mat_col * mat_type->vector_elements + vec_elem];
						}
						ir_constant* new_vec_const = new(parse_state) ir_constant(mat_type->column_type(), &new_vec_data);
						assign->insert_before(new(parse_state) ir_assignment(new(parse_state) ir_dereference_variable(mv.v[mat_col]), new_vec_const, assign->condition, 0xF));
					}
				}

				assign->remove();
			}
			else
			{
				check(false);//Should have removed matrix swizzles by now
			}
		}

		return ir_rvalue_visitor::visit_leave(assign);
	}

	virtual ir_visitor_status visit(ir_variable* var)
	{
		if (var->type->is_matrix())
		{
			MatrixVectors* mv = MatrixVectorMap.Find(var);
			if (!mv)
			{
				mv = &MatrixVectorMap.Add(var);
				void* p = ralloc_parent(var);

				const glsl_type* vtype = var->type->column_type();

				const char* base_name = var->name ? var->name : "temp";
				const char* name = ralloc_asprintf(p, "%s_%s", base_name, "col0");
				mv->v[0] = new(p) ir_variable(vtype, name, var->mode);
				name = ralloc_asprintf(p, "%s_%s", base_name, "col1");
				mv->v[1] = new(p) ir_variable(vtype, name, var->mode);
				name = ralloc_asprintf(p, "%s_%s", base_name, "col2");
				mv->v[2] = new(p) ir_variable(vtype, name, var->mode);
				name = ralloc_asprintf(p, "%s_%s", base_name, "col3");
				mv->v[3] = new(p) ir_variable(vtype, name, var->mode);

				//This is called manually on uniforms too but these are not in a cclist so those functions need not (and cannot) be called.
				if (var->mode != ir_var_uniform)
				{
					var->insert_before(mv->v[0]);
					var->insert_before(mv->v[1]);
					var->insert_before(mv->v[2]);
					var->insert_before(mv->v[3]);
					var->remove();
				}
			}
		}
		return visit_continue;
	}

	static void run(exec_list* ir, _mesa_glsl_parse_state* state)
	{
		bool progress = false;
		do
		{
			ir_matrices_to_vectors v(state);

			visit_list_elements(&v, ir);

			progress = v.progress;

			//vm_debug_print("== propagate non expressions - BEFORE DEADCODE ==\n");
			//vm_debug_dump(ir, state);

			//progress = do_dead_code(ir, false) || progress;
			//progress = do_dead_code_local(ir) || progress;
		} while (progress);
	}
};

void vm_matrices_to_vectors(exec_list* ir, _mesa_glsl_parse_state* state)
{
	ir_matrices_to_vectors::run(ir, state);
}

//////////////////////////////////////////////////////////////////////////


/** Replaces any assignments that reference themselves with ones that assign to a new temporary. It is potentially unsafe for the VM to read and write to the same location. */
class ir_rem_self_ref_assignments_visitor final : public ir_rvalue_visitor
{
	_mesa_glsl_parse_state* parse_state;

	typedef TTuple<ir_variable*, unsigned int> replace_info;

	/** Map of variables we should replace in subsequent rvalues. */
	TMap<replace_info, replace_info> to_replace;

	/* The current assignment we're in. */
	ir_assignment* curr_assign;

	replace_info just_replaced;

	bool progress;

public:
	ir_rem_self_ref_assignments_visitor(_mesa_glsl_parse_state* in_state)
	{
		parse_state = in_state;
		progress = false;

		curr_assign = nullptr;;

		just_replaced = replace_info(nullptr, 0);
	}

	virtual ~ir_rem_self_ref_assignments_visitor()
	{
	}

	unsigned get_component_from_matrix_array_deref(ir_dereference_array* array_deref)
	{
		check(array_deref);
		check(array_deref->variable_referenced()->type->is_matrix());
		ir_constant* index = array_deref->array_index->as_constant();
		check((index->type == glsl_type::uint_type || index->type == glsl_type::int_type) && index->type->is_scalar());
		unsigned deref_idx = index->type == glsl_type::uint_type ? index->value.u[0] : index->value.i[0];
		return deref_idx * array_deref->variable_referenced()->type->vector_elements;
	}

	unsigned get_dest_comp(unsigned write_mask)
	{
		if (write_mask == (1 << 0)) return 0;
		if (write_mask == (1 << 1)) return 1;
		if (write_mask == (1 << 2)) return 2;
		if (write_mask == (1 << 3)) return 3;
		check(0);
		return 0;
	}

	virtual void handle_rvalue(ir_rvalue** rvalue)
	{
		if (!rvalue || !*rvalue || in_assignee)
		{
			return;
		}

		ir_dereference* deref = (*rvalue)->as_dereference();
		ir_dereference_array* array_deref = (*rvalue)->as_dereference_array();
		ir_swizzle* swiz = (*rvalue)->as_swizzle();

		ir_variable* search_var = (*rvalue)->variable_referenced();
		unsigned search_comp = 0;
		if (swiz)
		{
			if (ir_dereference_array* swiz_array_deref = swiz->val->as_dereference_array())
			{
				search_comp = get_component_from_matrix_array_deref(swiz_array_deref);
			}
			search_comp += swiz->mask.x;
		}
		else if (array_deref)
		{
			//We can only handle matrix array derefs but these will have an outer swizzle that we'll work with. 
			check(array_deref->array->type->is_matrix());
			search_var = nullptr;
		}
		else if (!deref || !deref->type->is_scalar())
		{
			//If we're not a deref or we're not a straight scalar deref then we should leave this alone.
			search_var = nullptr;
		}

		replace_info search_var_comp(search_var, search_comp);
		replace_info* replace_var_comp = to_replace.Find(search_var_comp);

		if (curr_assign && (just_replaced == search_var_comp))
		{
			// if we're in an assign and we've had to replace the LHS, then don't worry about changing rvalues for
			// the search_var_comp
			return;
		}
		else if (replace_var_comp)
		{
			ir_rvalue* new_rval = new(parse_state) ir_dereference_variable(replace_var_comp->Key);
			(*rvalue) = new_rval;
			progress = true;
		}
		else if (curr_assign)
		{
			unsigned assign_comp = get_dest_comp(curr_assign->write_mask);
			if (curr_assign->lhs->variable_referenced() == search_var_comp.Key && assign_comp == search_var_comp.Value)
			{
				//We're in an assignment that is reading from the same component it's writing to.
				checkSlow(search_var);

				//Create a new temporary for the assignment and add it to the replacement map so that future refs to it are replaced.
				ir_variable* new_var = new(parse_state) ir_variable(search_var->type->get_base_type(), "self_assign_replacement", ir_var_temporary);
				replace_info& new_info = to_replace.Add(search_var_comp, replace_info(new_var, assign_comp));

				just_replaced = replace_info(search_var, search_comp);

				curr_assign->insert_before(new_var);
				// we want to reset the write_mask since we're now assigning the rhs to a scalar on the lhs
				curr_assign->write_mask = 1;
				curr_assign->set_lhs(new(parse_state) ir_dereference_variable(new_var));
				progress = true;
			}
		}
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign) override
	{
		if (assign->condition)
		{
			_mesa_glsl_error(parse_state, "conditional assignment in instruction stream");
			return visit_stop;
		}

		curr_assign = assign;
		just_replaced = replace_info(nullptr, 0);
		return visit_continue;
	}

	virtual ir_visitor_status visit_leave(ir_assignment* assign)
	{
		if (curr_assign)
		{
			// when assigning a variable we need to remove it from our to_replace map; clearing out the history
			// for the scenario:
			//	A = A + B;
			//	C = A;
			//	A = D;
			//	E = A
			// 
			// we want to adjust things to be:
			//	self_assign_replacement = A + B;
			//	C = self_assign_replacement;
			//	A = D;
			//	E = A;
			// 
			// this removal ensures that the 3rd line in the above code will ensure that A is removed from the history
			// allowing: E = A; instead of E = self_assign_replacement;
			to_replace.Remove(replace_info(curr_assign->lhs->variable_referenced(), get_dest_comp(curr_assign->write_mask)));
		}

		curr_assign = nullptr;
		just_replaced = replace_info(nullptr, 0);
		return visit_continue;
	}

	static void run(exec_list* ir, _mesa_glsl_parse_state* state)
	{
		bool progress = false;
		do
		{
			ir_rem_self_ref_assignments_visitor vis(state);
			visit_list_elements(&vis, ir);

			progress = vis.progress;

			progress = do_dead_code(ir, false) || progress;
			progress = do_dead_code_local(ir) || progress;
		} while (progress);
	}
};

void vm_remove_self_ref_assignments(exec_list* ir, _mesa_glsl_parse_state* state)
{
	ir_rem_self_ref_assignments_visitor::run(ir, state);
}