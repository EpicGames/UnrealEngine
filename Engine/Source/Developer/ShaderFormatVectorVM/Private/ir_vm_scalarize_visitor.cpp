// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

ECallScalarizeMode get_scalarize_mode(ir_function_signature* in_sig)
{
	EVectorVMOp SpecialOp = get_special_vm_opcode(in_sig);
	if (SpecialOp != EVectorVMOp::done)
	{
		bool scalar_ret = in_sig->return_type->is_scalar() || in_sig->return_type->is_void();
		bool no_structs = true;
		bool all_scalar = true;
		unsigned max_components = 0;
		unsigned min_components = 999;
		foreach_iter(exec_list_iterator, iter, in_sig->parameters)
		{
			ir_variable *var = (ir_variable *)iter.get();
			all_scalar &= var->type->is_scalar();
			no_structs &= var->type->base_type != GLSL_TYPE_STRUCT;
			max_components = FMath::Max(max_components, var->type->components());
			min_components = FMath::Min(min_components, var->type->components());
		}

		if (all_scalar && scalar_ret)
		{
			return ECallScalarizeMode::None;
		}

		if (scalar_ret)
		{
			//Can only split up prams into scalars, not a return type.
			return ECallScalarizeMode::SplitParams;
		}
		else
		{
			if (no_structs /*&& max_components == min_components*/)
			{
				return ECallScalarizeMode::SplitCalls;
			}
		}
	}
	else
	{
		return ECallScalarizeMode::None;
	}

	//_mesa_glsl_error(, "Cannot scalarize call %s", in_sig->function_name());
	return ECallScalarizeMode::Error;
}

/** splits all assignments into a separate assignment for each of it's components. Eventually down to the individual scalars. */
class ir_scalarize_visitor2 : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state *parse_state;

	unsigned dest_component;

	ir_rvalue* curr_rval;

	bool is_struct;

	bool has_split;

	ir_scalarize_visitor2(_mesa_glsl_parse_state *in_state)
		: parse_state(in_state)
		, dest_component(0)
		, curr_rval(nullptr)
		, is_struct(false)
		, has_split(false)
	{
	}

	virtual ~ir_scalarize_visitor2()
	{
	}

	virtual ir_visitor_status visit(ir_constant *ir)
	{
		check(ir->type->is_numeric() || ir->type->is_boolean());

		unsigned use_component = FMath::Min(ir->type->components(), dest_component);

		switch (ir->type->base_type)
		{
		case GLSL_TYPE_FLOAT:
			ir->value.f[0] = ir->value.f[use_component];
			ir->type = glsl_type::float_type;
			break;
		case GLSL_TYPE_INT:
			ir->value.i[0] = ir->value.i[use_component];
			ir->type = glsl_type::int_type;
			break;
		case GLSL_TYPE_UINT:
			ir->value.u[0] = ir->value.u[use_component];
			ir->type = glsl_type::uint_type;
			break;
		case GLSL_TYPE_BOOL:
			ir->value.b[0] = ir->value.b[use_component];
			ir->type = glsl_type::bool_type;
			break;
		}
		return visit_continue;
	}

	ir_function_signature* FindScalarSig(ir_function_signature* in_sig, bool bCreateIfNotFound)
	{
		//Ugh. Make less bad.
		ir_function* func = const_cast<ir_function*>(in_sig->function());

		//void* parent = ralloc_parent(in_sig);
		ECallScalarizeMode mode = get_scalarize_mode(in_sig);

		ir_function_signature* new_sig = nullptr;
		if (mode == ECallScalarizeMode::None)
		{
			return in_sig;
		}
		else if (mode == ECallScalarizeMode::SplitParams)
		{
			new_sig = new(parse_state) ir_function_signature(in_sig->return_type);
			check(in_sig->is_builtin);
			new_sig->is_builtin = true;

			TFunction<void(const ir_variable*, const glsl_type*, FString)> append_scalar_params = [&](const ir_variable* original, const glsl_type* type, FString Name)
			{
				const glsl_type* base_type = type->get_base_type();
				if (type->is_scalar())
				{
					new_sig->parameters.push_tail(new(parse_state) ir_variable(type, TCHAR_TO_ANSI(*Name), original->mode));
				}
				else if (type->is_vector())
				{
					append_scalar_params(original, base_type, Name + TEXT("_x"));
					append_scalar_params(original, base_type, Name + TEXT("_y"));
					if (type->vector_elements > 2)
					{
						append_scalar_params(original, base_type, Name + TEXT("_z"));
					}
					if (type->vector_elements > 3)
					{
						append_scalar_params(original, base_type, Name + TEXT("_w"));
					}
				}
				else if (type->is_matrix())
				{
					const char* matrix_names[4] = { "_Row0", "_Row1", "_Row2", "_Row3" };
					unsigned rows = type->vector_elements;
					for (unsigned r = 0; r < rows; ++r)
					{
						append_scalar_params(original, type->row_type(), Name + matrix_names[r]);
					}
				}
				else if (type->base_type == GLSL_TYPE_STRUCT)
				{
					for (unsigned member_idx = 0; member_idx < type->length; ++member_idx)
					{
						glsl_struct_field& struct_field = type->fields.structure[member_idx];
						append_scalar_params(original, struct_field.type, Name + TEXT("_") + struct_field.name);
					}
				}
				else
				{
					check(0);
				}
			};

			foreach_iter(exec_list_iterator, iter, in_sig->parameters)
			{
				ir_variable *var = (ir_variable *)iter.get();
				append_scalar_params(var, var->type, FString(var->name));
			}
		}
		else if (mode == ECallScalarizeMode::SplitCalls)
		{
			new_sig = new(parse_state) ir_function_signature(in_sig->return_type->get_base_type());
			foreach_iter(exec_list_iterator, iter, in_sig->parameters)
			{
				ir_variable *var = (ir_variable *)iter.get();
				new_sig->parameters.push_tail(new(parse_state) ir_variable(var->type->get_base_type(), var->name, var->mode));
			}
		}

		if (new_sig)
		{
			ir_function_signature* existing_sig = nullptr;
			foreach_iter(exec_list_iterator, SigIter, *func)
			{
				ir_function_signature* sig = (ir_function_signature *)SigIter.get();

				bool bEquivelentSigs = true;//AreEquivalent(sig, new_sig);//TODO: Implement direct in hlslcc

											//bEquivelentSigs &= sig->function() == new_sig->function();
				bEquivelentSigs &= sig->return_type == new_sig->return_type;
				exec_list_iterator sig_param_itr = sig->parameters.iterator();
				exec_list_iterator new_sig_param_itr = new_sig->parameters.iterator();

				for (; sig_param_itr.has_next() && new_sig_param_itr.has_next(); sig_param_itr.next(), new_sig_param_itr.next())
				{
					ir_variable* param = (ir_variable*)sig_param_itr.get();
					ir_variable* new_param = (ir_variable*)new_sig_param_itr.get();

					bool bParamsEquiv = strcmp(param->name, new_param->name) == 0;
					bParamsEquiv &= param->mode == new_param->mode;
					bParamsEquiv &= param->type == new_param->type;
					//Need more?
					if (!bParamsEquiv)
					{
						bEquivelentSigs = false;
						break;
					}
				}
				bEquivelentSigs &= !sig_param_itr.has_next() && !new_sig_param_itr.has_next();

				//no need to check body here
				//Other stuff would need checking for proper impl

				if (bEquivelentSigs)
				{
					existing_sig = sig;
					break;
				}
			}

			if (existing_sig)
			{
				ralloc_free(new_sig);
				return existing_sig;
			}
			else if (bCreateIfNotFound)
			{
				func->add_signature(new_sig);
				return new_sig;
			}

			ralloc_free(new_sig);
		}

		return nullptr;
	}
	
	virtual ir_visitor_status visit_enter(ir_call* call)
	{
		//void* parent = ralloc_parent(call->callee);
		ECallScalarizeMode mode = get_scalarize_mode(call->callee);

		check(call->next && call->prev);

		ir_function_signature* scalar_sig = FindScalarSig(call->callee, true);
		
		if (scalar_sig)
		{
			if (mode == ECallScalarizeMode::SplitParams)
			{
				call->callee = scalar_sig;
				exec_list old_params = call->actual_parameters;

				call->actual_parameters.make_empty();

				foreach_iter(exec_list_iterator, param_iter, old_params)
				{
					ir_rvalue* param = (ir_rvalue*)param_iter.get();
					dest_component = 0;

					TFunction<void(ir_rvalue*)> add_scalar_param = [&](ir_rvalue* rval)
					{
						if (rval->type->is_scalar())
						{
							call->actual_parameters.push_tail(rval);
						}
						else
						{
							unsigned old_dest_comp = dest_component;
							unsigned num_components = rval->type->base_type == GLSL_TYPE_STRUCT ? rval->type->length : rval->type->components();
							dest_component = 0;
							while (dest_component < num_components)
							{
								check(rval);
								rval->accept(this);
								check(curr_rval);

								add_scalar_param(curr_rval);
							}
							dest_component = old_dest_comp;
						}

						++dest_component;//Move to next component in current context.
					};

					add_scalar_param(param);
				}
			}
			else if (mode == ECallScalarizeMode::SplitCalls)
			{
				unsigned max_components = 0;
				foreach_iter(exec_list_iterator, param_iter, call->actual_parameters)
				{
					ir_rvalue* param = (ir_rvalue *)param_iter.get();
					check(param->type->base_type != GLSL_TYPE_STRUCT);
					max_components = FMath::Max(max_components, param->type->components());
				}

				//Clone the call max_component times and visit each param to scalarize them to the right component.				
				ir_variable* old_dest = call->return_deref->var;
				//void* perm_mem_ctx = ralloc_parent(call);
				for (unsigned i = 0; i < max_components; ++i)
				{
					dest_component = i;

					ir_variable* new_dest = new(parse_state) ir_variable(old_dest->type->get_base_type(), old_dest->name, ir_var_temporary);
					call->insert_before(new_dest);

					exec_list new_params;
					foreach_iter(exec_list_iterator, param_iter, call->actual_parameters)
					{
						ir_rvalue* param = (ir_rvalue*)param_iter.get();
						param = param->clone(parse_state, nullptr);
						curr_rval = nullptr;
						param->accept(this);
						new_params.push_tail(curr_rval ? curr_rval : param);
					}

					call->insert_before(new(parse_state)ir_call(scalar_sig, new(parse_state) ir_dereference_variable(new_dest), &new_params));

					ir_dereference* deref = call->return_deref->clone(parse_state, nullptr);
					unsigned write_mask = 1 << dest_component;
					call->insert_before(new(parse_state)ir_assignment(deref, new(parse_state) ir_dereference_variable(new_dest), nullptr, write_mask));
				}

				call->remove();
			}
		}
		else
		{
			_mesa_glsl_error(parse_state, "could not find a scalar signature for function %s", call->callee->function_name());
			return visit_stop;
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign)
	{
		if (assign->condition)
		{
			_mesa_glsl_error(parse_state, "conditional assignment in instruction stream");
			return visit_stop;
		}

		check(assign->next && assign->prev);

		//void* perm_mem_ctx = ralloc_parent(assign);

		const glsl_type* type = assign->lhs->type;

		//Already scalar or writes to just one component. Bail.
		unsigned num_components_written = 0;
		unsigned write_mask = assign->write_mask;
		while (write_mask > 0)
		{
			num_components_written += write_mask & 0x1 ? 1 : 0;
			write_mask = write_mask >> 1;
		}

		if (type->is_scalar() || num_components_written == 1)
		{
			return visit_continue_with_parent;
		}

		has_split = true;

		is_struct = type->base_type == GLSL_TYPE_STRUCT;
		check(is_struct || type->is_matrix() || type->is_vector());

		unsigned num_components = is_struct ? type->length : type->components();

		write_mask = assign->write_mask == 0 ? 0xFFFFFFFF : assign->write_mask;
		ir_assignment* comp_assign = NULL;

		unsigned comp_idx = 0;
		for (unsigned write_index = 0; write_index < num_components; ++write_index)
		{
			if (is_struct || (write_mask & (1 << write_index)))
			{
				if (comp_assign)
				{
					assign->insert_before(comp_assign);
					//Ensure that we have a valid write mask if we're splitting down a structure assignment.
					if (comp_assign->lhs->type->is_scalar())
					{
						comp_assign->write_mask = 1;
					}
					comp_assign = NULL;
				}

				comp_assign = assign->clone(parse_state, NULL);
				check(comp_assign);
				dest_component = comp_idx;

				if (is_struct)
				{
					comp_assign->write_mask = 0;
					comp_assign->set_lhs(new(parse_state)ir_dereference_record(comp_assign->lhs, comp_assign->lhs->type->fields.structure[dest_component].name));
				}
				else
				{
					comp_assign->write_mask = 1 << write_index;
				}

				curr_rval = nullptr;
				comp_assign->rhs->accept(this);
				if (curr_rval)
				{
					comp_assign->rhs = curr_rval;
				}

				++comp_idx;
			}
		}

		if (comp_assign)
		{
			assign->replace_with(comp_assign);
			//Ensure that we have a valid write mask if we're splitting down a structure assignment.
			if (comp_assign->lhs->type->is_scalar())
			{
				comp_assign->write_mask = 1;
			}
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_swizzle* swiz)
	{
		check(!is_struct);//We shouldn't be hitting a swizzle if we're splitting a struct assignment.

	//	if (swiz->mask.num_components > 1)
		{
			//check(dest_component < swiz->mask.num_components);
			unsigned use_component = FMath::Min(swiz->mask.num_components, dest_component);
			check(swiz->val->as_swizzle() == nullptr);//Handling swizzles of swizzles is possible but avoid if possible.

			unsigned src_comp = 0;
			switch (use_component)
			{
			case 0: src_comp = swiz->mask.x; break;
			case 1: src_comp = swiz->mask.y; break;
			case 2: src_comp = swiz->mask.z; break;
			case 3: src_comp = swiz->mask.w; break;
			default:check(0);
				break;
			}

			swiz->mask.num_components = 1;
			swiz->mask.has_duplicates = false;
			swiz->type = swiz->type->get_base_type();

			if (ir_constant* const_val = swiz->val->as_constant())
			{
				//The scalar itself may be scalar but it's internals could be a non scalar constant.
				//In that case we visit the constant and it'll convert itself to the right scalar based on dest_componnent.
				unsigned old_dest = dest_component;
				dest_component = src_comp;
				swiz->val->accept(this);
				dest_component = old_dest;

				swiz->mask.x = 0;
			}
			else
			{
				swiz->mask.x = src_comp;
			}
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_dereference_array* array_deref)
	{
		//void* perm_mem_ctx = ralloc_parent(array_deref);
		const glsl_type* array_type = array_deref->array->type;
		const glsl_type* type = array_deref->type;

		//Only supporting array derefs for matrices atm.
		check(array_type->is_matrix());
		check(type->is_vector());

		//Only support constant matrix indices. Not immediately clear how I'd do non-const access.
		ir_constant* index = array_deref->array_index->as_constant();
		check(index->type == glsl_type::uint_type || index->type == glsl_type::int_type);
		check(index->type->is_scalar());

// 		unsigned mat_comp = index->type == glsl_type::uint_type ? index->value.u[0] : index->value.i[0];
// 		unsigned swiz_comp = mat_comp * array_type->vector_elements + dest_component;
// 		ir_dereference_variable* new_deref = new(perm_mem_ctx) ir_dereference_variable(array_deref->variable_referenced());
// 		ir_swizzle* swiz = new(perm_mem_ctx) ir_swizzle(new_deref, type->is_scalar() ? 0 : swiz_comp, 0, 0, 0, 1);
// 		curr_rval = swiz;

		ir_swizzle* swiz = new(parse_state) ir_swizzle(array_deref, dest_component, 0, 0, 0, 1);
		curr_rval = swiz;

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_dereference_record* deref)
	{
		//void* perm_mem_ctx = ralloc_parent(deref);
		const glsl_type* type = deref->type;

		if (type->base_type == GLSL_TYPE_STRUCT)
		{
			check(dest_component < type->length);

			ir_dereference_record* rec = new(parse_state) ir_dereference_record(deref, type->fields.structure[dest_component].name);
			curr_rval = rec;
		}
		else
		{
			check(deref->type->is_numeric() || deref->type->is_boolean());
			//check(type->components() >= dest_component || type->is_scalar());
			unsigned use_component = FMath::Min(type->components(), dest_component);

			ir_swizzle* swiz = new(parse_state) ir_swizzle(deref, type->is_scalar() ? 0 : use_component, 0, 0, 0, 1);
			curr_rval = swiz;
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit(ir_dereference_variable *deref)
	{
		//void* perm_mem_ctx = ralloc_parent(deref);
		ir_variable* var = deref->variable_referenced();
		const glsl_type* type = var->type;
		if (type->base_type == GLSL_TYPE_STRUCT)
		{
			check(dest_component < type->length);

			ir_dereference_record* rec = new(parse_state) ir_dereference_record(var, type->fields.structure[dest_component].name);
			curr_rval = rec;
		}
		else
		{
			if (!type->is_scalar())
			{
				check(var->type->is_numeric() || var->type->is_boolean());
				//check(type->components() > dest_component);
				unsigned use_component = FMath::Min(type->components(), dest_component);

				ir_swizzle* swiz = new(parse_state) ir_swizzle(deref, use_component, 0, 0, 0, 1);
				curr_rval = swiz;
			}
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_expression *expr)
	{
		ir_rvalue* old_rval = curr_rval;

		if (is_struct)
		{
			check(dest_component < expr->type->length);
			expr->type = expr->type->fields.structure[dest_component].type;
		}
		else
		{
			expr->type = expr->type->get_base_type();
		}

		unsigned int num_operands = expr->get_num_operands();

		//visit all operands and replace any rvalues with duplicates that only access the dest_component of that var.
		for (unsigned op = 0; op < num_operands; ++op)
		{
			curr_rval = NULL;
			expr->operands[op]->accept(this);

			if (curr_rval)
			{
				expr->operands[op] = curr_rval;
			}
		}

		curr_rval = old_rval;
		return visit_continue_with_parent;
	}

public:
	static void run(exec_list *ir, _mesa_glsl_parse_state *state)
	{
		bool progress = false;
		do
		{
			ir_scalarize_visitor2 scalarize_visitor(state);
			visit_list_elements(&scalarize_visitor, ir);
			progress = scalarize_visitor.has_split;
		} while (progress);

		do
		{
			//We can now split up structures as everything is accessed by individual components. Makes subsequent visitors simpler.
			progress = do_structure_splitting(ir, state);

			progress = do_dead_code(ir, false);			
			//This falls over on some matrix swizzles so leave it out for now.
			//progress = do_dead_code_local(ir) || progress;
		} while (progress);
	}
};


void vm_scalarize_ops(exec_list* ir, _mesa_glsl_parse_state* state)
{
	//Generate the noise signatures. Maybe should move noise handling to another visitor.		

	//Find the noise func.
	ir_function* noise_func = nullptr;
	foreach_iter(exec_list_iterator, iter, *ir)
	{
		if (ir_function* noise = ((ir_instruction*)iter.get())->as_function())
		{
			if (strcmp(noise->name, "noise") == 0)
			{
				noise_func = noise;
				break;
			}			
		}
	}
	if (noise_func)
	{
		ir_function_signature* noise1 = nullptr;

		//Noise1 sig may already exist so find it.
		foreach_iter(exec_list_iterator, iter, *noise_func)
		{
			ir_function_signature* sig = (ir_function_signature *)iter.get();

			bool scalar = true;
			unsigned num_params = 0;
			foreach_iter(exec_list_iterator, iterInner, sig->parameters)
			{
				scalar &= ((ir_rvalue*)iterInner.get())->type->is_scalar();
				++num_params;
			}
			if (num_params == 1 && scalar)
			{
				noise1 = sig;
			}
		}

		if (!noise1)
		{
			ir_function_signature* new_sig = new(state)ir_function_signature(glsl_type::float_type);
			new_sig->is_builtin = true;
			new_sig->has_output_parameters = false;
			new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "x", ir_var_in));
			noise_func->add_signature(new_sig);
		}

		ir_function_signature* new_sig = new(state)ir_function_signature(glsl_type::float_type);
		new_sig->is_builtin = true;
		new_sig->has_output_parameters = false;
		new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "x", ir_var_in));
		new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "y", ir_var_in));
		noise_func->add_signature(new_sig);

		new_sig = new(state)ir_function_signature(glsl_type::float_type);
		new_sig->is_builtin = true;
		new_sig->has_output_parameters = false;
		new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "x", ir_var_in));
		new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "y", ir_var_in));
		new_sig->parameters.push_tail(new(state)ir_variable(glsl_type::float_type, "z", ir_var_in));
		noise_func->add_signature(new_sig);

		//ir->push_tail(noise_func);
	}

	ir_scalarize_visitor2::run(ir, state);
}