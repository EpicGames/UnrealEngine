// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"


class ir_to_single_op_visitor2 final : ir_hierarchical_visitor
{
	_mesa_glsl_parse_state* state;
	bool assign_has_expressions;

	bool progress;

	TArray<ir_instruction*> expression_stack;

	//Generated replacement for the current expression.
	ir_rvalue* replacement;

public:
	ir_to_single_op_visitor2(_mesa_glsl_parse_state* in_state)
	{
		state = in_state;
		assign_has_expressions = false;
		progress = false;
		replacement = nullptr;
	}

	virtual ~ir_to_single_op_visitor2()
	{
	}

	virtual ir_visitor_status visit_enter(ir_swizzle* swiz) override
	{
		expression_stack.Push(swiz);
		swiz->val->accept(this);

		if (replacement)
		{
			swiz->val = replacement;
			replacement = nullptr;
			progress = true;
		}
		expression_stack.Pop();

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_expression* expr) override
	{
		check(base_ir->next && base_ir->prev);
		assign_has_expressions = true;
		expression_stack.Push(expr);
		for (unsigned i = 0; i < expr->get_num_operands(); ++i)
		{
			expr->operands[i]->accept(this);

			if (replacement)
			{
				expr->operands[i] = replacement;
				replacement = nullptr;
				progress = true;
			}
		}
		expression_stack.Pop();

		unsigned expr_depth = expression_stack.Num();
		ir_instruction* parent = expr_depth > 0 ? expression_stack.Top() : nullptr;
		//void* parent_ctx = ralloc_parent(expr);

		if (parent)
		{
			ir_variable *tmp_var = new(state) ir_variable(expr->type, "tmp_var", ir_var_temporary);
			ir_assignment *tmp_assign = new(state) ir_assignment(new(state) ir_dereference_variable(tmp_var), expr);
			check(tmp_assign->write_mask > 0);
			base_ir->insert_before(tmp_var);
			base_ir->insert_before(tmp_assign);
			replacement = new(state) ir_dereference_variable(tmp_var);
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_assignment* assign) override
	{
		check(base_ir->next && base_ir->prev);
		assign_has_expressions = false;
		assign->rhs->accept(this);

		check(replacement == nullptr);
		if (assign->lhs->variable_referenced()->mode == ir_var_out && assign_has_expressions)
		{
			//void *mem_ctx = ralloc_parent(assign);
			ir_variable *tmp_var = new(state) ir_variable(assign->rhs->type, "output_var", ir_var_temporary);
			ir_assignment *tmp_assign = new(state) ir_assignment(new(state) ir_dereference_variable(tmp_var), assign->rhs);
			check(tmp_assign->write_mask > 0);
			base_ir->insert_before(tmp_var);
			base_ir->insert_before(tmp_assign);
			assign->rhs = new(state) ir_dereference_variable(tmp_var);
			progress = true;
		}

		return visit_continue_with_parent;
	}

	virtual ir_visitor_status visit_enter(ir_call* call) override
	{
		check(base_ir->next && base_ir->prev);
		assign_has_expressions = true;
		//void *mem_ctx = ralloc_parent(call);

		foreach_iter(exec_list_iterator, iter, call->actual_parameters)
		{
			ir_rvalue* param = (ir_rvalue*)iter.get();

			ir_dereference* deref = param->as_dereference();
			if (!deref)
			{
				//This param isn't just a deref so move it out to it's own variable and deref it in the call instead.	
				ir_variable *tmp_var = new(state) ir_variable(param->type, "call_param_temp", ir_var_temporary);
				ir_assignment *tmp_assign = new(state) ir_assignment(new(state) ir_dereference_variable(tmp_var), param);
				check(tmp_assign->write_mask > 0);
				base_ir->insert_before(tmp_var);
				base_ir->insert_before(tmp_assign);
				ir_rvalue* new_param = new(state) ir_dereference_variable(tmp_var);
				check(param->next && param->prev);
				param->replace_with(new_param);
				param = new_param;
				progress = true;
			}
		}

		return visit_continue_with_parent;
	}

	static void run(exec_list *ir, _mesa_glsl_parse_state *state)
	{
		bool progress = false;
		do
		{
			ir_to_single_op_visitor2 to_single_op_visitor(state);
			visit_list_elements(&to_single_op_visitor, ir);
			progress = to_single_op_visitor.progress;
		} while (progress);
	}
};


void vm_to_single_op(exec_list *ir, _mesa_glsl_parse_state *state)
{
	ir_to_single_op_visitor2::run(ir, state);
}


