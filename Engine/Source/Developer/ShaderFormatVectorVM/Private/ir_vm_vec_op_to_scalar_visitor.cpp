// Copyright Epic Games, Inc. All Rights Reserved.


#include "ShaderFormatVectorVM.h"
#include "CoreMinimal.h"
#include "hlslcc.h"
#include "hlslcc_private.h"
#include "VectorVMBackend.h"

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
#include "glsl_parser_extras.h"
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

#include "hash_table.h"
#include "ir_rvalue_visitor.h"
#include "PackUniformBuffers.h"
#include "IRDump.h"
#include "OptValueNumbering.h"
#include "ir_optimization.h"
#include "ir_expression_flattening.h"
#include "ir.h"

#include "VectorVM.h"

//Additional helper during scalaization to split any special vector expressions such as dot into their scalar operations.
class ir_vec_op_to_scalar_visitor : public ir_hierarchical_visitor
{
	_mesa_glsl_parse_state* state;

public:
	ir_vec_op_to_scalar_visitor(_mesa_glsl_parse_state* in_state)
	{
		state = in_state;
		this->made_progress = false;
	}

	ir_dereference* get_deref_or_create_temp(ir_rvalue* rval, ir_dereference* result)
	{
		ir_dereference *deref = rval->as_dereference();

		check(base_ir->next && base_ir->prev);

		//void* parent = ralloc_parent(base_ir);
		/* Avoid making a temporary if we don't need to to avoid aliasing. */
		if (deref &&
			deref->variable_referenced() != result->variable_referenced())
		{
			return deref;			
		}

		/* Otherwise, store the operand in a temporary generally if it's
		* not a dereference.
		*/
		ir_variable *var = new(state)ir_variable(rval->type,
			"vec_op_to_scalar",
			ir_var_temporary);
		base_ir->insert_before(var);

		/* Note that we use this dereference for the assignment.  That means
		* that others that want to use op[i] have to clone the deref.
		*/
		deref = new(state)ir_dereference_variable(var);

		ir_assignment *assign = new(state)ir_assignment(deref, rval);
		check(assign->write_mask > 0);
		base_ir->insert_before(assign);
		return deref;
	}

	ir_visitor_status visit_leave(ir_call * call)
	{
		if (strcmp(call->callee_name(), "length") != 0)
		{
			return visit_continue;
		}

		ir_rvalue* param = nullptr;
		foreach_iter(exec_list_iterator, param_iter, call->actual_parameters)
		{
			check(param == nullptr); //length should only have one param!
			param = ((ir_rvalue *)param_iter.get());
		}

		ir_dereference_variable *result = call->return_deref;
		check(result);

		ir_dereference* deref = get_deref_or_create_temp(param, result);

		do_length(result, deref);

		check(call->next && call->prev);
		call->remove();

		return visit_continue;
	}

	ir_visitor_status visit_leave(ir_assignment * orig_assign)
	{
		//void* parent = ralloc_parent(base_ir);
		ir_expression *orig_expr = orig_assign->rhs->as_expression();
		ir_dereference *op[2] = {nullptr, nullptr};

		if (!orig_expr)
		{
			return visit_continue;
		}

		//Skip any ops that aren't vector specific ops we're replacing.
		//Suprised length() isn't an ir op...
		if (orig_expr->operation != ir_binop_dot && orig_expr->operation != ir_binop_cross && orig_expr->operation != ir_unop_normalize)
		{
			return visit_continue;
		}

		ir_dereference_variable *result = orig_assign->lhs->as_dereference_variable();
		check(result);

		for (unsigned i = 0; i < orig_expr->get_num_operands(); i++)
		{
			op[i] = get_deref_or_create_temp(orig_expr->operands[i], result);
		}
		check(op[0]);
		switch (orig_expr->operation)
		{
		case ir_binop_dot: check(op[1]); do_dot(result, op[0], op[1]); break;
		case ir_binop_cross: check(op[1]); do_cross(result, op[0], op[1]); break;
		case ir_unop_normalize: do_normalize(result, op[0]); break;
		};

		check(orig_assign->next && orig_assign->prev);
		orig_assign->remove();
		return visit_continue;
	}

	void do_dot(ir_dereference *result, ir_dereference *a, ir_dereference *b);
	void do_cross(ir_dereference *result, ir_dereference *a, ir_dereference *b);
	void do_normalize(ir_dereference *result, ir_dereference *a);
	void do_length(ir_dereference* result, ir_dereference* a);

	bool made_progress;
};

void ir_vec_op_to_scalar_visitor::do_dot(ir_dereference *result, ir_dereference *a, ir_dereference *b)
{
	ir_assignment *assign;
	ir_expression *expr;
	check(base_ir->next && base_ir->prev);
	check(a->type == b->type);
	//void* parent = ralloc_parent(base_ir);

	expr = new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
		new(state)ir_swizzle(a, 0, 0, 0, 0, 1),
		new(state)ir_swizzle(b, 0, 0, 0, 0, 1));

	for (unsigned comp = 1; comp < a->type->vector_elements; comp++)
	{
		expr = new(state)ir_expression(ir_binop_add, a->type->get_base_type(),
			expr,
			new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
				new(state)ir_swizzle(a, comp, 0, 0, 0, 1),
				new(state)ir_swizzle(b, comp, 0, 0, 0, 1))
		);
	}

	assign = new(state)ir_assignment(result, expr);
	check(assign->write_mask > 0);
	base_ir->insert_before(assign);
}

void ir_vec_op_to_scalar_visitor::do_cross(ir_dereference *result, ir_dereference *a, ir_dereference *b)
{
	ir_assignment *assign;
	check(base_ir->next && base_ir->prev);
	check(a->type == b->type);
	check(a->type->is_vector());
	check(a->type->vector_elements == 3);
	const glsl_type* type = a->type;
	const glsl_type* basetype = a->type->get_base_type();

	//void* parent = ralloc_parent(base_ir);

#define GETX(v) (new(state)ir_swizzle(v, 0, 0, 0, 0, 1))
#define GETY(v) (new(state)ir_swizzle(v, 1, 0, 0, 0, 1))
#define GETZ(v) (new(state)ir_swizzle(v, 2, 0, 0, 0, 1))
#define MUL(A,B)  (new(state)ir_expression(ir_binop_mul, basetype, A, B))
#define SUB(A,B)  (new(state)ir_expression(ir_binop_sub, basetype, A, B))

	assign = new(state)ir_assignment(result, SUB(MUL(GETY(a), GETZ(b)), MUL(GETZ(a), GETY(b))));
	assign->write_mask = 1;
	base_ir->insert_before(assign);

	assign = new(state)ir_assignment(result, SUB(MUL(GETZ(a), GETX(b)), MUL(GETX(a), GETZ(b))));
	assign->write_mask = 1 << 1;
	base_ir->insert_before(assign);

	assign = new(state)ir_assignment(result, SUB(MUL(GETX(a), GETY(b)), MUL(GETY(a), GETX(b))));
	assign->write_mask = 1 << 2;
	base_ir->insert_before(assign);

#undef GETX
#undef GETY
#undef GETZ
#undef MUL
#undef SUB
}

void ir_vec_op_to_scalar_visitor::do_normalize(ir_dereference *result, ir_dereference *a)
{
	ir_assignment *assign;
	ir_expression *expr;
	check(base_ir->next && base_ir->prev);
	check(a->type->is_vector());

	//void* parent = ralloc_parent(base_ir);

	//generate the dot of a with itself.
	expr = new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
		new(state)ir_swizzle(a, 0, 0, 0, 0, 1),
		new(state)ir_swizzle(a, 0, 0, 0, 0, 1));

	for (unsigned comp = 1; comp < a->type->vector_elements; comp++)
	{
		expr = new(state)ir_expression(ir_binop_add, a->type->get_base_type(),
			expr,
			new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
				new(state)ir_swizzle(a, comp, 0, 0, 0, 1),
				new(state)ir_swizzle(a, comp, 0, 0, 0, 1))
		);
	}

	//Generate the length and divide a by it.
	ir_expression* inv_len = new(state)ir_expression(ir_unop_rsq, a->type->get_base_type(), expr);
	expr = new(state)ir_expression(ir_binop_mul, a->type, a, inv_len);

	assign = new(state)ir_assignment(result, expr);
	check(assign->write_mask > 0);
	base_ir->insert_before(assign);
}

void ir_vec_op_to_scalar_visitor::do_length(ir_dereference *result, ir_dereference *a)
{
	ir_assignment *assign;
	ir_expression *expr;
	check(base_ir->next && base_ir->prev);
	check(a->type->is_vector());

	//void* parent = ralloc_parent(base_ir);

	//generate the dot of a with itself.
	expr = new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
		new(state)ir_swizzle(a, 0, 0, 0, 0, 1),
		new(state)ir_swizzle(a, 0, 0, 0, 0, 1));

	for (unsigned comp = 1; comp < a->type->vector_elements; comp++)
	{
		expr = new(state)ir_expression(ir_binop_add, a->type->get_base_type(),
			expr,
			new(state)ir_expression(ir_binop_mul, a->type->get_base_type(),
				new(state)ir_swizzle(a, comp, 0, 0, 0, 1),
				new(state)ir_swizzle(a, comp, 0, 0, 0, 1))
		);
	}

	expr = new(state)ir_expression(ir_unop_sqrt, a->type->get_base_type(), expr);
	assign = new(state)ir_assignment(result, expr);
	check(assign->write_mask > 0);
	base_ir->insert_before(assign);
}

static bool vec_op_to_scalar_predicate(ir_instruction *ir)
{
	ir_expression *expr = ir->as_expression();
	ir_call *call = ir->as_call();

	if (expr)
	{
		if (expr->operation == ir_binop_dot || expr->operation == ir_binop_cross || expr->operation == ir_unop_normalize)
			return true;
	}
	else if (call)
	{
		return strcmp(call->callee_name(), "length") == 0;
	}

	return false;
}

bool do_vec_op_to_scalar(exec_list *instructions, _mesa_glsl_parse_state* parse_state)
{
	ir_vec_op_to_scalar_visitor v(parse_state);

	do_expression_flattening(instructions, vec_op_to_scalar_predicate);

	visit_list_elements(&v, instructions);

	return v.made_progress;
}