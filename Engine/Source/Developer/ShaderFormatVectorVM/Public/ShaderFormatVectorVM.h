// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "VectorVM.h"

struct FShaderCompilerInput;
struct FShaderCompilerOutput;

/** Ordered table of functions actually called by the VM script. */
struct FCalledVMFunction
{
	FString Name;
	TArray<bool> InputParamLocations;
	int32 NumOutputs;
	FCalledVMFunction() :NumOutputs(0) {}
};

/** Data which is generated from the hlsl by the VectorVMBackend and fed back into the */
struct FVectorVMCompilationOutput
{
	FVectorVMCompilationOutput(): MaxTempRegistersUsed(0), NumOps(0) {}

	TArray<uint8> ByteCode;

	int32 MaxTempRegistersUsed;

	TArray<int32> InternalConstantOffsets;
	TArray<uint8> InternalConstantData;
	TArray<EVectorVMBaseTypes> InternalConstantTypes;
	
	TArray<FCalledVMFunction> CalledVMFunctionTable;

	FString AssemblyAsString;
	uint32 NumOps;

	FString Errors;
};

inline FArchive& operator<<(FArchive& Ar, FCalledVMFunction& Function)
{
	Ar << Function.Name;
	Ar << Function.InputParamLocations;
	Ar << Function.NumOutputs;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FVectorVMCompilationOutput& Output)
{
	Ar << Output.ByteCode;
	Ar << Output.MaxTempRegistersUsed;
	Ar << Output.InternalConstantOffsets;
	Ar << Output.InternalConstantData;
	Ar << Output.InternalConstantTypes;
	Ar << Output.CalledVMFunctionTable;
	Ar << Output.AssemblyAsString;
	Ar << Output.NumOps;
	Ar << Output.Errors;
	return Ar;
}
