// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ShaderFormatVectorVM : ModuleRules
{
	public ShaderFormatVectorVM(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform",
				}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderCompilerCommon",
				"ShaderPreprocessor",
				"VectorVM",
				}
			);

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"HLSLCC"
			);
	}
}
